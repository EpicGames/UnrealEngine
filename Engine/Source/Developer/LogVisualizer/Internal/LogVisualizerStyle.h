// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Templates/SharedPointer.h"

class ISlateStyle;
class FSlateStyleSet;

#define UE_API LOGVISUALIZER_API

/**
 * Implements the visual style of LogVisualizer.
 */
class FLogVisualizerStyle
{
public:
	static void Initialize();
	static void Shutdown();
	UE_API static const ISlateStyle& Get();
	UE_API static FName GetStyleSetName();

private:
	static TSharedRef<FSlateStyleSet> Create();

	static TSharedPtr<FSlateStyleSet> StyleInstance;
};

#undef UE_API
