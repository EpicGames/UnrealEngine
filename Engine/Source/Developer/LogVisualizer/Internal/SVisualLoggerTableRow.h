// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "LogVisualizerStyle.h"
#include "Templates/SharedPointer.h"
#include "VisualLoggerTypes.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

#define UE_API LOGVISUALIZER_API

class SVisualLoggerTableRow : public SMultiColumnTableRow<TSharedPtr<FLogEntryItem>>
{
public:
	SLATE_BEGIN_ARGS(SVisualLoggerTableRow)
	{
	}
		SLATE_ATTRIBUTE(FText, FilterText)
	SLATE_END_ARGS()

	UE_API static const FLazyName ColumnId_CategoryLabel;
	UE_API static const FLazyName ColumnId_VerbosityLabel;
	UE_API static const FLazyName ColumnId_LogLabel;

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedPtr<FLogEntryItem> InEntryItem)
	{
		Item = InEntryItem;
		FilterText = InArgs._FilterText;
		SMultiColumnTableRow::Construct(FSuperRowType::FArguments(), InOwnerTableView);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		if (ColumnName == ColumnId_CategoryLabel)
		{
			return SNew(STextBlock)
				.ColorAndOpacity(FSlateColor(Item->CategoryColor))
				.Text(FText::FromString(Item->Category))
				.HighlightText(this, &SVisualLoggerTableRow::GetFilterText);
		}

		 if (ColumnName == ColumnId_VerbosityLabel)
		{
			return SNew(STextBlock)
				.ColorAndOpacity(FSlateColor(Item->Verbosity == ELogVerbosity::Error ? FLinearColor::Red : (Item->Verbosity == ELogVerbosity::Warning ? FLinearColor::Yellow : FLinearColor::Gray)))
				.Text(FText::FromString(FString(TEXT("(")) + FString(::ToString(Item->Verbosity)) + FString(TEXT(")"))));
		}

		if (ColumnName == ColumnId_LogLabel)
		{
			return SNew(STextBlock)
				.AutoWrapText(true)
				.ColorAndOpacity(FSlateColor(Item->Verbosity == ELogVerbosity::Error ? FLinearColor::Red : (Item->Verbosity == ELogVerbosity::Warning ? FLinearColor::Yellow : FLinearColor::Gray)))
				.Text(FText::FromString(Item->Line))
				.HighlightText(this, &SVisualLoggerTableRow::GetFilterText)
				.TextStyle(FLogVisualizerStyle::Get(), TEXT("TextLogs.Text"));
		}

		return SNullWidget::NullWidget;
	}

	FText GetFilterText() const
	{
		return FilterText.Get();
	}

private:
	TAttribute<FText> FilterText;
	TSharedPtr<FLogEntryItem> Item;
};

#undef UE_API
