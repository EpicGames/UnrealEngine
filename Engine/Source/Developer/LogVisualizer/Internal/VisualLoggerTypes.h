// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Math/Color.h"
#include "Logging/LogVerbosity.h"
#include "UObject/NameTypes.h"

struct FLogEntryItem
{
	FString Category;
	FLinearColor CategoryColor = FLinearColor::White;
	ELogVerbosity::Type Verbosity = ELogVerbosity::Log;
	FString Line;
	int64 UserData = 0;
	FName TagName = NAME_None;
};
