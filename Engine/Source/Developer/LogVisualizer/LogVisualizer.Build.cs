// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class LogVisualizer : ModuleRules
{
	public LogVisualizer(ReadOnlyTargetRules Target) : base(Target)
	{
        PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"DesktopPlatform",
			}
		);

		PublicIncludePathModuleNames.Add("WorkspaceMenuStructure");

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"ApplicationCore",
				"InputCore",
				"Json",
				"Slate",
				"SlateCore",
				
				"Engine",
				"EditorFramework",
				"UnrealEd",
			}
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"DesktopPlatform",
				"MainFrame",
			}
		);
	}
}
