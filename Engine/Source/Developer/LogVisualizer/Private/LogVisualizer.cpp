// Copyright Epic Games, Inc. All Rights Reserved.

#include "LogVisualizer.h"

#if UE_DEBUG_VISUALIZER_TOOL_ENABLED
#include "Engine/GameViewportClient.h"
#include "Engine/LocalPlayer.h"
#include "EngineUtils.h"
#include "GameFramework/SpectatorPawn.h"
#include "LogVisualizerPublic.h"
#include "LogVisualizerSettings.h"
#include "VisualLogger/VisualLogger.h"
#include "VisualLoggerCameraController.h"
#include "VisualLoggerDatabase.h"
#include "VisualLoggerTimeSliderController.h"

#if WITH_EDITOR
#include "Editor/EditorEngine.h"
#include "EditorViewportClient.h"
#endif

TSharedPtr< struct FLogVisualizer > FLogVisualizer::StaticInstance;
FColor FLogVisualizerColorPalette[] = {
	FColor(0xff8fbc8f), // darkseagreen
	FColor(0xff48d1cc), // mediumturquoise
	FColor(0xffffa500), // orange
	FColor(0xff1e90ff), // dodgerblue
	FColor(0xffffff00), // yellow
	FColor(0xffdeb887), // burlywood
	FColor(0xff00ff00), // lime
	FColor(0xffdc143c), // crimson
	FColor(0xff00fa9a), // mediumspringgreen
	FColor(0xff00bfff), // deepskyblue
	FColor(0xffadff2f), // greenyellow
	FColor(0xffff00ff), // fuchsia
	FColor(0xfff0e68c), // khaki
	FColor(0xffdda0dd), // plum
	FColor(0xff90ee90), // lightgreen
	FColor(0xffff1493), // deeppink
	FColor(0xffffa07a), // lightsalmon
	FColor(0xffee82ee), // violet
	FColor(0xff7fffd4), // aquamarine
	FColor(0xffff0000), // red
	FColor(0xfffafad2), // lightgoldenrod
	FColor(0xffcd5c5c), // indianred
	FColor(0xffe6e6fa), // lavender
	FColor(0xffffb6c1), // lightpink
	FColor(0xffa9a9a9), // darkgray
	FColor(0xff9932cc), // darkorchid
	FColor(0xff556b2f), // darkolivegreen
	FColor(0xffb03060), // maroon3
	FColor(0xff8b4513), // saddlebrown
	FColor(0xff228b22), // forestgreen
	FColor(0xff6a5acd), // slateblue
	FColor(0xff808000), // olive
	FColor(0xff3cb371), // mediumseagreen
	FColor(0xffb8860b), // darkgoldenrod
	FColor(0xff008b8b), // darkcyan
	FColor(0xff4682b4), // steelblue
	FColor(0xff32cd32), // limegreen
	FColor(0xffd2691e), // chocolate
	FColor(0xff9acd32), // yellowgreen
	FColor(0xff191970), // midnightblue
	FColor(0xff8b0000), // darkred
	FColor(0xff0000ff), // blue
	FColor(0xff00008b), // darkblue
	FColor(0xff7f007f), // purple2
	FColor(0xff2f4f4f), // darkslategray
};


void FLogVisualizer::Initialize()
{
	StaticInstance = MakeShareable(new FLogVisualizer);
	Get().TimeSliderController = MakeShareable(new FVisualLoggerTimeSliderController(FVisualLoggerTimeSliderArgs()));
}

void FLogVisualizer::Shutdown()
{
	StaticInstance.Reset();
}

void FLogVisualizer::Reset()
{
	TimeSliderController->SetTimesliderArgs(FVisualLoggerTimeSliderArgs());

#if UE_DEBUG_RECORDING_USING_VLOG
	FVisualLogger::Get().OnDataReset();
#endif
}

FLogVisualizer& FLogVisualizer::Get()
{
	return *StaticInstance;
}

FLinearColor FLogVisualizer::GetColorForCategory(int32 Index) const
{
	if (Index >= 0 && Index < sizeof(FLogVisualizerColorPalette) / sizeof(FLogVisualizerColorPalette[0]))
	{
		return FLogVisualizerColorPalette[Index];
	}

	static bool bReateColorList = false;
	static FColorList StaticColor;
	if (!bReateColorList)
	{
		bReateColorList = true;
		StaticColor.CreateColorMap();
	}
	return StaticColor.GetFColorByIndex(Index);
}

FLinearColor FLogVisualizer::GetColorForCategory(const FString& InFilterName) const
{
	static TArray<FString> Filters;
	int32 CategoryIndex = Filters.Find(InFilterName);
	if (CategoryIndex == INDEX_NONE)
	{
		CategoryIndex = Filters.Add(InFilterName);
	}

	return GetColorForCategory(CategoryIndex);
}

UWorld* FLogVisualizer::GetWorld(UObject* OptionalObject)
{
	UWorld* World = GEngine->GetWorldFromContextObject(OptionalObject, EGetWorldErrorMode::ReturnNull);
#if WITH_EDITOR
	if (!World && GIsEditor)
	{
		UEditorEngine* EEngine = Cast<UEditorEngine>(GEngine);
		// lets use PlayWorld during PIE/Simulate and regular world from editor otherwise, to draw debug information
		World = EEngine != nullptr && EEngine->PlayWorld != nullptr ? ToRawPtr(EEngine->PlayWorld) : EEngine->GetEditorWorldContext().World();
	}
	else 
#endif
	if (!World && !GIsEditor)
	{
		World = GEngine->GetWorld();
	}

	if (World == nullptr)
	{
		World = GWorld;
	}

	return World;
}

void FLogVisualizer::UpdateCameraPosition(FName RowName, int32 ItemIndes)
{
	const FVisualLoggerDBRow& DBRow = FVisualLoggerDatabase::Get().GetRowByName(RowName);
	const TArray<FVisualLogDevice::FVisualLogEntryItem>& Entries = DBRow.GetItems();
	if (DBRow.GetCurrentItemIndex() == INDEX_NONE || Entries.IsValidIndex(DBRow.GetCurrentItemIndex()) == false)
	{
		return;
	}

	UWorld* World = GetWorld();
	
	FVector CurrentLocation = Entries[DBRow.GetCurrentItemIndex()].Entry.Location;

	FVector Extent(150);
	bool bFoundActor = false;
	FName OwnerName = Entries[DBRow.GetCurrentItemIndex()].OwnerName;
	for (FActorIterator It(World); It; ++It)
	{
		AActor* Actor = *It;
		if (Actor->GetFName() == OwnerName)
		{
			FVector Orgin;
			Actor->GetActorBounds(false, Orgin, Extent);
			bFoundActor = true;
			break;
		}
	}


	const float DefaultCameraDistance = ULogVisualizerSettings::StaticClass()->GetDefaultObject<ULogVisualizerSettings>()->DefaultCameraDistance;
	Extent = Extent.SizeSquared() < FMath::Square(DefaultCameraDistance) ? FVector(DefaultCameraDistance) : Extent;

#if WITH_EDITOR
	UEditorEngine *EEngine = Cast<UEditorEngine>(GEngine);
	if (GIsEditor && EEngine != NULL)
	{
		for (auto ViewportClient : EEngine->GetAllViewportClients())
		{
			ViewportClient->FocusViewportOnBox(FBox::BuildAABB(CurrentLocation, Extent));
		}
	}
	else if (AVisualLoggerCameraController::IsEnabled(World) && AVisualLoggerCameraController::Instance.IsValid() && AVisualLoggerCameraController::Instance->GetSpectatorPawn())
	{
		ULocalPlayer* LocalPlayer = Cast<ULocalPlayer>(AVisualLoggerCameraController::Instance->Player);
		if (LocalPlayer && LocalPlayer->ViewportClient && LocalPlayer->ViewportClient->Viewport)
		{

			FViewport* Viewport = LocalPlayer->ViewportClient->Viewport;

			FBox BoundingBox = FBox::BuildAABB(CurrentLocation, Extent);
			const FVector Position = BoundingBox.GetCenter();
			float Radius = static_cast<float>(BoundingBox.GetExtent().Size());
			
			FViewportCameraTransform ViewTransform;
			ViewTransform.TransitionToLocation(Position, nullptr, true);

			float NewOrthoZoom;
			const float AspectRatio = 1.777777f;
			CA_SUPPRESS(6326);
			uint32 MinAxisSize = (AspectRatio > 1.0f) ? Viewport->GetSizeXY().Y : Viewport->GetSizeXY().X;
			const float Zoom = Radius / (MinAxisSize / 2.0f);

			NewOrthoZoom = Zoom * (Viewport->GetSizeXY().X*15.0f);
			NewOrthoZoom = FMath::Clamp<float>(NewOrthoZoom, 250.0f, MAX_FLT);
			ViewTransform.SetOrthoZoom(NewOrthoZoom);

			AVisualLoggerCameraController::Instance->GetSpectatorPawn()->TeleportTo(ViewTransform.GetLocation(), ViewTransform.GetRotation(), false, true);
		}
	}
#endif
}

int32 FLogVisualizer::GetNextItem(FName RowName, int32 MoveDistance)
{
	FVisualLoggerDBRow& DBRow = FVisualLoggerDatabase::Get().GetRowByName(RowName);
	int32 NewItemIndex = DBRow.GetCurrentItemIndex();

	int32 Index = 0;
	const TArray<FVisualLogDevice::FVisualLogEntryItem>& Entries = DBRow.GetItems();
	while (true)
	{
		NewItemIndex++;
		if (Entries.IsValidIndex(NewItemIndex))
		{
			if (DBRow.IsItemVisible(NewItemIndex) == true && ++Index == MoveDistance)
			{
				break;
			}
		}
		else
		{
			NewItemIndex = FMath::Clamp(NewItemIndex, 0, Entries.Num() - 1);
			break;
		}
	}

	return NewItemIndex;
}

int32 FLogVisualizer::GetPreviousItem(FName RowName, int32 MoveDistance)
{
	FVisualLoggerDBRow& DBRow = FVisualLoggerDatabase::Get().GetRowByName(RowName);
	int32 NewItemIndex = DBRow.GetCurrentItemIndex();

	int32 Index = 0;
	const TArray<FVisualLogDevice::FVisualLogEntryItem>& Entries = DBRow.GetItems();
	while (true)
	{
		NewItemIndex--;
		if (Entries.IsValidIndex(NewItemIndex))
		{
			if (DBRow.IsItemVisible(NewItemIndex) == true && ++Index == MoveDistance)
			{
				break;
			}
		}
		else
		{
			NewItemIndex = FMath::Clamp(NewItemIndex, 0, Entries.Num() - 1);
			break;
		}
	}
	return NewItemIndex;
}

void FLogVisualizer::GotoNextItem(FName RowName, int32 MoveDistance)
{
	FVisualLoggerDBRow& DBRow = FVisualLoggerDatabase::Get().GetRowByName(RowName);
	const int32 NewItemIndex = GetNextItem(RowName, MoveDistance);

	if (NewItemIndex != DBRow.GetCurrentItemIndex())
	{
		TimeSliderController->CommitScrubPosition(DBRow.GetItems()[NewItemIndex].Entry.TimeStamp, /*bIsScrubbing*/false);
	}
}

void FLogVisualizer::GotoPreviousItem(FName RowName, int32 MoveDistance)
{
	FVisualLoggerDBRow& DBRow = FVisualLoggerDatabase::Get().GetRowByName(RowName);
	const int32 NewItemIndex = GetPreviousItem(RowName, MoveDistance);

	if (NewItemIndex != DBRow.GetCurrentItemIndex())
	{
		TimeSliderController->CommitScrubPosition(DBRow.GetItems()[NewItemIndex].Entry.TimeStamp, /*bIsScrubbing*/false);
	}
}

void FLogVisualizer::GotoFirstItem(FName RowName)
{
	FVisualLoggerDBRow& DBRow = FVisualLoggerDatabase::Get().GetRowByName(RowName);
	int32 NewItemIndex = DBRow.GetCurrentItemIndex();

	const TArray<FVisualLogDevice::FVisualLogEntryItem>& Entries = DBRow.GetItems();
	for (int32 Index = 0; Index <= DBRow.GetCurrentItemIndex(); Index++)
	{
		if (DBRow.IsItemVisible(Index))
		{
			NewItemIndex = Index;
			break;
		}
	}

	if (NewItemIndex != DBRow.GetCurrentItemIndex())
	{
		//DBRow.MoveTo(NewItemIndex);
		TimeSliderController->CommitScrubPosition(Entries[NewItemIndex].Entry.TimeStamp, /*bIsScrubbing*/false);
	}
}

void FLogVisualizer::GotoLastItem(FName RowName)
{
	FVisualLoggerDBRow& DBRow = FVisualLoggerDatabase::Get().GetRowByName(RowName);
	int32 NewItemIndex = DBRow.GetCurrentItemIndex();

	const TArray<FVisualLogDevice::FVisualLogEntryItem>& Entries = DBRow.GetItems();
	for (int32 Index = Entries.Num() - 1; Index >= DBRow.GetCurrentItemIndex(); Index--)
	{
		if (DBRow.IsItemVisible(Index))
		{
			NewItemIndex = Index;
			break;
		}
	}

	if (NewItemIndex != DBRow.GetCurrentItemIndex())
	{
		//DBRow.MoveTo(NewItemIndex);
		TimeSliderController->CommitScrubPosition(Entries[NewItemIndex].Entry.TimeStamp, /*bIsScrubbing*/false);
	}
}

void FLogVisualizer::GotoLastItemAnyRow()
{
	double LastItemTimeStamp = -1.0;
	for (FVisualLoggerDatabase::FConstRowIterator Iter(FVisualLoggerDatabase::Get().GetConstRowIterator()); Iter; ++Iter)
	{
		const TArray<FVisualLogDevice::FVisualLogEntryItem>& Entries = Iter->GetItems();
		for (int32 ItemIndex = Entries.Num() - 1; ItemIndex >= 0; ItemIndex--)
		{
			if (Iter->IsItemVisible(ItemIndex))
			{
				if (Entries[ItemIndex].Entry.TimeStamp > LastItemTimeStamp)
				{
					LastItemTimeStamp = Entries[ItemIndex].Entry.TimeStamp;
					break;
				}
			}
		}
	}

	if (LastItemTimeStamp > -1.0)
	{
		TimeSliderController->CommitScrubPosition(LastItemTimeStamp, /*bIsScrubbing*/false);
	}
}

void FLogVisualizer::SeekToTime(float Time)
{
	GetTimeSliderController()->CommitScrubPosition(Time,  /*bIsScrubbing=*/true);
}
#endif // UE_DEBUG_VISUALIZER_TOOL_ENABLED