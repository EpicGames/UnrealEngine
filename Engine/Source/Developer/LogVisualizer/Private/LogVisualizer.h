// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once


#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "EngineDefines.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "VisualLogger/VisualLoggerDefines.h"

#if UE_DEBUG_VISUALIZER_TOOL_ENABLED

/* Dependencies
*****************************************************************************/

#include "GenericPlatform/GenericPlatformFile.h"
#include "Layout/ArrangedChildren.h"
#include "Textures/TextureAtlas.h"
#include "Brushes/SlateDynamicImageBrush.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SToolTip.h"
#include "Dom/JsonObject.h"
#include "Framework/MultiBox/MultiBox.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

/* Private includes
*****************************************************************************/

#include "GameFramework/DamageType.h"
#include "TimerManager.h"
#include "HAL/RunnableThread.h"
#include "Engine/World.h"
#include "SVisualLogger.h"
#include "SVisualLoggerToolbar.h"
#include "SVisualLoggerFilters.h"
#include "Widgets/Layout/SScrollBarTrack.h"
#include "SVisualLoggerView.h"
#include "SVisualLoggerLogsList.h"
#include "SVisualLoggerStatusView.h"
#include "SVisualLoggerTimeline.h"
#include "SVisualLoggerTimelineBar.h"
#include "MovieScene.h"
#include "VisualLoggerTimeSliderController.h"
#endif // UE_DEBUG_VISUALIZER_TOOL_ENABLED
