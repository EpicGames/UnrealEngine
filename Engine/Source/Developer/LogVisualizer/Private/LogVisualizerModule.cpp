// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Textures/SlateIcon.h"
#include "Framework/Docking/TabManager.h"
#include "ILogVisualizer.h"
#include "LogVisualizerSettings.h"
#include "VisualLoggerDatabase.h"
#include "LogVisualizerStyle.h"
#include "SVisualLogger.h"
#include "SVisualLoggerTab.h"
#include "VisualLoggerCommands.h"
#include "Widgets/Docking/SDockTab.h"
#include "LogVisualizerPublic.h"
#include "Features/IModularFeatures.h"

#if WITH_EDITOR
#include "ISettingsModule.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"
#endif // WITH_EDITOR

#define LOCTEXT_NAMESPACE "FLogVisualizerModule"

static const FName VisualLoggerTabName("VisualLogger");

//DEFINE_LOG_CATEGORY(LogLogVisualizer);

class FLogVisualizerModule : public ILogVisualizer
{
public:
	// Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	// End IModuleInterface

private:
	TSharedRef<SDockTab> SpawnLogVisualizerTab(const FSpawnTabArgs& SpawnTabArgs);
};

void FLogVisualizerModule::StartupModule()
{
	FLogVisualizerStyle::Initialize();
	FVisualLoggerDatabase::Initialize();
	FLogVisualizer::Initialize();
	FVisualLoggerFilters::Initialize();

	FVisualLoggerCommands::Register();
	IModularFeatures::Get().RegisterModularFeature(VisualLoggerTabName, this);

	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(
		VisualLoggerTabName, 
		FOnSpawnTab::CreateRaw(this, &FLogVisualizerModule::SpawnLogVisualizerTab))
		.SetGroup(WorkspaceMenu::GetMenuStructure().GetDeveloperToolsDebugCategory())
		.SetDisplayName(NSLOCTEXT("LogVisualizerApp", "TabTitle", "Visual Logger"))
		.SetTooltipText(NSLOCTEXT("LogVisualizerApp", "TooltipText", "Opens Visual Logger tool."))
		.SetIcon(FSlateIcon(FLogVisualizerStyle::GetStyleSetName(), "LogVisualizerApp.TabIcon"));

	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
	if (SettingsModule != nullptr)
	{
		SettingsModule->RegisterSettings("Editor", "Advanced", "VisualLogger",
			LOCTEXT("AIToolsSettingsName", "Visual Logger"),
			LOCTEXT("AIToolsSettingsDescription", "General settings for UE AI Tools."),
			ULogVisualizerSettings::StaticClass()->GetDefaultObject()
			);
	}
}

void FLogVisualizerModule::ShutdownModule()
{
	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(VisualLoggerTabName);
	FVisualLoggerCommands::Unregister();
	IModularFeatures::Get().UnregisterModularFeature(VisualLoggerTabName, this);
	
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
	if (SettingsModule != nullptr)
	{
		SettingsModule->UnregisterSettings("Editor", "General", "VisualLogger");
	}

	FVisualLoggerFilters::Shutdown();
	FLogVisualizer::Shutdown();
	FVisualLoggerDatabase::Shutdown();
	FLogVisualizerStyle::Shutdown();
}

TSharedRef<SDockTab> FLogVisualizerModule::SpawnLogVisualizerTab(const FSpawnTabArgs& SpawnTabArgs)
{
	const TSharedRef<SDockTab> MajorTab = SNew(SVisualLoggerTab)
		.TabRole(ETabRole::NomadTab);

	TSharedPtr<SWidget> TabContent;

	TabContent = SNew(SVisualLogger, MajorTab, SpawnTabArgs.GetOwnerWindow());

	MajorTab->SetContent(TabContent.ToSharedRef());

	return MajorTab;
}

IMPLEMENT_MODULE(FLogVisualizerModule, LogVisualizer);
#undef LOCTEXT_NAMESPACE
