// Copyright Epic Games, Inc. All Rights Reserved.

#include "LogVisualizerSessionSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LogVisualizerSessionSettings)

#if WITH_EDITOR
#include "UnrealEdMisc.h"
#endif // WITH_EDITOR

ULogVisualizerSessionSettings::ULogVisualizerSessionSettings(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	bEnableGraphsVisualization = false;
}

#if WITH_EDITOR
void ULogVisualizerSessionSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName Name = (PropertyChangedEvent.Property != nullptr) ? PropertyChangedEvent.Property->GetFName() : NAME_None;
#if 0  //FIXME: should we save this settings too? (SebaK)
	if (!FUnrealEdMisc::Get().IsDeletePreferences())
	{
		SaveConfig();
	}
#endif
	SettingChangedEvent.Broadcast(Name);
}
#endif

