// Copyright Epic Games, Inc. All Rights Reserved.

#include "LogVisualizerSettings.h"
#include "Materials/Material.h"
#include "VisualLogger/VisualLoggerTypes.h"
#include "VisualLoggerDatabase.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LogVisualizerSettings)

#if WITH_EDITOR
#include "UnrealEdMisc.h"
#endif // WITH_EDITOR

ULogVisualizerSettings::ULogVisualizerSettings(const FObjectInitializer& ObjectInitializer) 
	: Super(ObjectInitializer)
	, DebugMeshMaterialFakeLightName(TEXT("/Engine/EngineDebugMaterials/DebugMeshMaterialFakeLight.DebugMeshMaterialFakeLight"))
{
	TrivialLogsThreshold = 1;
	DefaultCameraDistance = 150;
	bSearchInsideLogs = true;
	bUseFilterVolumes = true;
	GraphsBackgroundColor = FColor(0, 0, 0, 70);
	bResetDataWithNewSession = false;
	bDrawExtremesOnGraphs = false;
	bUsePlayersOnlyForPause = true;
	bConstrainGraphToLocalMinMax = false;
}

class UMaterial* ULogVisualizerSettings::GetDebugMeshMaterial()
{
	if (DebugMeshMaterialFakeLight == nullptr)
	{
		DebugMeshMaterialFakeLight = LoadObject<UMaterial>(NULL, *DebugMeshMaterialFakeLightName, NULL, LOAD_None, NULL);
	}

	return DebugMeshMaterialFakeLight;
}

void ULogVisualizerSettings::SavePersistentData()
{
	if (bPersistentFilters)
	{
		PersistentFilters = FVisualLoggerFilters::Get();
		for (int32 Index = PersistentFilters.Categories.Num() - 1; Index >= 0; --Index)
		{
			FCategoryFilter& Category = PersistentFilters.Categories[Index];
			if (Category.bIsInUse == false)
			{
				PersistentFilters.Categories.RemoveAt(Index);
			}
		}
	}
	else
	{
		PersistentFilters = FVisualLoggerFilters();
	}
	SaveConfig();
}

void ULogVisualizerSettings::ClearPersistentData()
{
	if (bPersistentFilters)
	{
		PersistentFilters = FVisualLoggerFilters();
	}
}

void ULogVisualizerSettings::LoadPersistentData()
{
	if (bPersistentFilters)
	{
		for (int32 Index = PersistentFilters.Categories.Num() - 1; Index >= 0; --Index)
		{
			FCategoryFilter& Category = PersistentFilters.Categories[Index];
			Category.bIsInUse = false;
		}
		FVisualLoggerFilters::Get().InitWith(PersistentFilters);
	}
	else
	{
		FVisualLoggerFilters::Get().Reset();
	}
}

void ULogVisualizerSettings::ConfigureVisLog()
{
	FVisualLogger::Get().SetUseUniqueNames(bForceUniqueLogNames);
}

#if WITH_EDITOR
void ULogVisualizerSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	static const FName NAME_bForceUniqueLogNames = GET_MEMBER_NAME_CHECKED(ULogVisualizerSettings, bForceUniqueLogNames);

	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName Name = (PropertyChangedEvent.Property != nullptr) ? PropertyChangedEvent.Property->GetFName() : NAME_None;
	if (!FUnrealEdMisc::Get().IsDeletePreferences())
	{
		SaveConfig();
	}
	if (Name == NAME_bForceUniqueLogNames)
	{
		FVisualLogger::Get().SetUseUniqueNames(bForceUniqueLogNames);
	}	

	SettingChangedEvent.Broadcast(Name);
}
#endif

#if UE_DEBUG_VISUALIZER_TOOL_ENABLED

//////////////////////////////////////////////////////////////////////////
// FVisualLoggerFilters
//////////////////////////////////////////////////////////////////////////
TSharedPtr< struct FVisualLoggerFilters > FVisualLoggerFilters::StaticInstance;

FVisualLoggerFilters& FVisualLoggerFilters::Get()
{
	return *StaticInstance;
}

void FVisualLoggerFilters::Initialize()
{
	StaticInstance = MakeShareable(new FVisualLoggerFilters);
	FVisualLoggerDatabase::Get().GetEvents().OnNewItem.AddRaw(StaticInstance.Get(), &FVisualLoggerFilters::OnNewItemHandler);
}

void FVisualLoggerFilters::Shutdown()
{
	FVisualLoggerDatabase::Get().GetEvents().OnNewItem.RemoveAll(StaticInstance.Get());
	StaticInstance.Reset();
}

void FVisualLoggerFilters::OnNewItemHandler(const FVisualLoggerDBRow& DBRow, int32 ItemIndex)
{
	const FVisualLogDevice::FVisualLogEntryItem& Item = DBRow.GetItems()[ItemIndex];
	TArray<FVisualLoggerCategoryVerbosityPair> VisualLoggerCategories;
	FVisualLoggerHelpers::GetCategories(Item.Entry, VisualLoggerCategories);
	for (auto& CategoryAndVerbosity : VisualLoggerCategories)
	{
		AddCategory(CategoryAndVerbosity.CategoryName.ToString(), ELogVerbosity::All);
	}

	TMap<FString, TArray<FString> > OutCategories;
	FVisualLoggerHelpers::GetHistogramCategories(Item.Entry, OutCategories);
	for (const auto& CurrentCategory : OutCategories)
	{
		for (const auto& CurrentDataName : CurrentCategory.Value)
		{
			const FString GraphFilterName = CurrentCategory.Key + TEXT("$") + CurrentDataName;	
			AddCategory(GraphFilterName, ELogVerbosity::All);
		}
	}
}

void FVisualLoggerFilters::AddCategory(FString InName, ELogVerbosity::Type InVerbosity)
{
	const int32 Num = Categories.Num();
	for (int32 Index = 0; Index < Num; ++Index)
	{
		FCategoryFilter& Filter = Categories[Index];
		if (Filter.CategoryName == InName)
		{
			Filter.bIsInUse = true;
			return;
		}
	}

	FCategoryFilter Filter;
	Filter.CategoryName = InName;
	Filter.LogVerbosity = InVerbosity;
	Filter.Enabled = true;
	Filter.bIsInUse = true;
	Categories.Add(Filter);

	FastCategoryFilterMap.Reset(); // we need to recreate cache - pointers can be broken
	FastCategoryFilterMap.Reserve(Categories.Num());
	for (int32 Index = 0; Index < Categories.Num(); Index++)
	{
		FastCategoryFilterMap.Add(*Categories[Index].CategoryName, &Categories[Index]);
	}

	OnFilterCategoryAdded.Broadcast(InName, InVerbosity);
}

void FVisualLoggerFilters::RemoveCategory(FString InName)
{
	for (int32 Index = 0; Index < Categories.Num(); ++Index)
	{
		const FCategoryFilter& Filter = Categories[Index];
		if (Filter.CategoryName == InName)
		{
			Categories.RemoveAt(Index);
			FastCategoryFilterMap.Remove(*InName);
			break;
		}
	}

	OnFilterCategoryRemoved.Broadcast(InName);
}

FCategoryFilter& FVisualLoggerFilters::GetCategoryByName(const FName& InName)
{
	FCategoryFilter* FilterPtr = FastCategoryFilterMap.Contains(InName) ? FastCategoryFilterMap[InName] : nullptr;
	if (FilterPtr)
	{
		return *FilterPtr;
	}

	static FCategoryFilter NoCategory;
	return NoCategory;
}

FCategoryFilter& FVisualLoggerFilters::GetCategoryByName(const FString& InName)
{
	const FName NameAsName = *InName;
	FCategoryFilter* FilterPtr = FastCategoryFilterMap.Contains(NameAsName) ? FastCategoryFilterMap[NameAsName] : nullptr;
	if (FilterPtr)
	{
		return *FilterPtr;
	}

	static FCategoryFilter NoCategory;
	return NoCategory;
}

bool FVisualLoggerFilters::MatchObjectName(FString String)
{
	return SelectedClasses.Num() == 0 || SelectedClasses.Find(String) != INDEX_NONE;
}


void FVisualLoggerFilters::SelectObject(FString ObjectName)
{
	SelectedClasses.AddUnique(ObjectName);
}

void FVisualLoggerFilters::RemoveObjectFromSelection(FString ObjectName)
{
	SelectedClasses.Remove(ObjectName);
}

const TArray<FString>& FVisualLoggerFilters::GetSelectedObjects() const
{
	return SelectedClasses;
}

bool FVisualLoggerFilters::ShouldDisplayLine(const FVisualLogLine& Line, const bool bSearchInsideLogs) const
{
	bool bShowLine = ShouldDisplayCategory(Line.Category, Line.Verbosity);
	if (bShowLine && bSearchInsideLogs)
	{
		const FString SearchString = GetSearchString();
		if (SearchString.Len() > 0
			&& Line.Line.Find(SearchString) == INDEX_NONE
			&& Line.Category.ToString().Find(SearchString) == INDEX_NONE)
		{
			bShowLine = false;
		}
	}
	return bShowLine;
}

bool FVisualLoggerFilters::ShouldDisplayCategory(const FName Name, const ELogVerbosity::Type Verbosity) const
{
	const ULogVisualizerSettings* Settings = ULogVisualizerSettings::StaticClass()->GetDefaultObject<ULogVisualizerSettings>();
	for (const FCategoryFilter& Filter : Categories)
	{
		if (Filter.CategoryName == Name)
		{
			if (Filter.Enabled)
			{
				const bool bFineWithSearchString = Settings->bSearchInsideLogs == true || (SearchBoxFilter.Len() == 0 || Filter.CategoryName.Find(SearchBoxFilter) != INDEX_NONE);
				return bFineWithSearchString && Verbosity <= Filter.LogVerbosity;
			}

			return false;
		}
	}

	// if filter for a given log category has not been found we let it be, we allow it.
	return true;
}

bool FVisualLoggerFilters::ShouldDisplayCategoryByString(const FStringView String, const ELogVerbosity::Type Verbosity) const
{
	const ULogVisualizerSettings* Settings = ULogVisualizerSettings::StaticClass()->GetDefaultObject<ULogVisualizerSettings>();
	for (const FCategoryFilter& Filter : Categories)
	{
		if (Filter.CategoryName.Equals(String.GetData(), ESearchCase::IgnoreCase))
		{
			if (Filter.Enabled)
			{
				const bool bFineWithSearchString = Settings->bSearchInsideLogs == true || (SearchBoxFilter.Len() == 0 || Filter.CategoryName.Find(SearchBoxFilter) != INDEX_NONE);
				return bFineWithSearchString && Verbosity <= Filter.LogVerbosity;
			}

			return false;
		}
	}

	// if filter for a given log category has not been found we let it be, we allow it.
	return true;
}

void FVisualLoggerFilters::DeactivateAllButThis(const FString& InName)
{
	for (FCategoryFilter& Filter : Categories)
	{
		Filter.Enabled = Filter.CategoryName == InName;
	}
}

void FVisualLoggerFilters::EnableAllCategories()
{
	for (FCategoryFilter& Filter : Categories)
	{
		Filter.Enabled = true;
	}
}

void FVisualLoggerFilters::Reset()
{
	for (int32 Index = Categories.Num() - 1; Index >= 0; --Index)
	{
		Categories[Index].bIsInUse = false;
	}

	SearchBoxFilter = FString();
	ObjectNameFilter = FString();

	SelectedClasses.Reset();
}

void FVisualLoggerFilters::InitWith(const FVisualLoggerFiltersData& NewFiltersData)
{
	SearchBoxFilter = NewFiltersData.SearchBoxFilter;
	ObjectNameFilter = NewFiltersData.ObjectNameFilter;
	Categories = NewFiltersData.Categories;
	SelectedClasses = NewFiltersData.SelectedClasses;

	FastCategoryFilterMap.Reset(); // we need to recreate cache - pointers can be broken
	FastCategoryFilterMap.Reserve(Categories.Num());
	for (int32 Index = 0; Index < Categories.Num(); Index++)
	{
		FastCategoryFilterMap.Add(*Categories[Index].CategoryName, &Categories[Index]);
	}
}

void FVisualLoggerFilters::DisableGraphData(FName GraphName, FName DataName, bool SetAsDisabled)
{
	const FName FullName = *(GraphName.ToString() + TEXT("$") + DataName.ToString());
	if (SetAsDisabled)
	{
		DisabledGraphDatas.AddUnique(FullName);
	}
	else
	{
		DisabledGraphDatas.RemoveSwap(FullName);
	}
}

bool FVisualLoggerFilters::IsGraphDataDisabled(FName GraphName, FName DataName)
{
	const FName FullName = *(GraphName.ToString() + TEXT("$") + DataName.ToString());
	return DisabledGraphDatas.Find(FullName) != INDEX_NONE;
}

#endif // UE_DEBUG_VISUALIZER_TOOL_ENABLED
