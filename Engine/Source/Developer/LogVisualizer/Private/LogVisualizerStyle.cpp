// Copyright Epic Games, Inc. All Rights Reserved.

#include "LogVisualizerStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Styling/StarshipCoreStyle.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleMacros.h"

// This is to fix the issue that SlateStyleMacros like IMAGE_BRUSH look for RootToContentDir but StyleSet->RootToContentDir is how this style is set up
#define RootToContentDir Style.RootToContentDir
#define RootToCoreContentDir Style.RootToCoreContentDir

TSharedPtr< FSlateStyleSet > FLogVisualizerStyle::StyleInstance = nullptr;

void FLogVisualizerStyle::Initialize()
{
	if (!StyleInstance.IsValid())
	{
		StyleInstance = Create();
		FSlateStyleRegistry::RegisterSlateStyle(*StyleInstance);
	}
}

void FLogVisualizerStyle::Shutdown()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*StyleInstance);
	ensure(StyleInstance.IsUnique());
	StyleInstance.Reset();
}

FName FLogVisualizerStyle::GetStyleSetName()
{
	static FName StyleSetName(TEXT("LogVisualizerStyle"));
	return StyleSetName;
}

TSharedRef< FSlateStyleSet > FLogVisualizerStyle::Create()
{
	const FVector2f Icon8x8(8.0f, 8.0f);
	const FVector2f Icon16x16(16.0f, 16.0f);
	const FVector2f Icon20x20(20.0f, 20.0f);
	const FVector2f Icon40x40(40.0f, 40.0f);
	const FVector2f Icon48x48(48.0f, 48.0f);

	TSharedRef<FSlateStyleSet> StyleRef = MakeShareable(new FSlateStyleSet(FLogVisualizerStyle::GetStyleSetName()));
	StyleRef->SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
	StyleRef->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	FSlateStyleSet& Style = StyleRef.Get();
	// Generic styles
	const FTextBlockStyle NormalText = FTextBlockStyle()
		.SetFont(DEFAULT_FONT("Regular", 9))
		.SetColorAndOpacity(FSlateColor::UseForeground())
		.SetShadowOffset(FVector2f::ZeroVector)
		.SetShadowColorAndOpacity(FLinearColor::Black)
		.SetHighlightColor(FLinearColor(0.02f, 0.3f, 0.0f))
		.SetHighlightShape(BOX_BRUSH("Common/TextBlockHighlightShape", FMargin(3.f / 8.f)));

	const FTextBlockStyle LogsText = FTextBlockStyle(NormalText)
		.SetFont(DEFAULT_FONT("Mono", 9));

	Style.Set("TextLogs.Text", LogsText);

	{
		Style.Set("LogVisualizerApp.TabIcon", new IMAGE_BRUSH_SVG("Starship/Common/VisualLogger", Icon16x16));
		
		Style.Set("LogVisualizer.LogBar.Background", new BOX_BRUSH("Common/ProgressBar_Background", FMargin(5.f / 12.f)));
		Style.Set("LogVisualizer.LogBar.Selected", new CORE_BOX_BRUSH("Common/ProfileVisualizer_Normal", FMargin(5.f / 12.f)));
		Style.Set("LogVisualizer.LogBar.EntryDefault", new CORE_BOX_BRUSH("Common/ProfileVisualizer_Mono", FMargin(5.f / 12.f)));
		Style.Set("LogVisualizer.LogBar.TimeMark", new BOX_BRUSH("Icons/LV_BarMark", FMargin(5.f / 12.f)));

		//Style.Set("ToolPanel.GroupBorder", FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"));
		Style.Set("ToolPanel.GroupBorder", new BOX_BRUSH("Common/GroupBorder", FMargin(4.0f / 16.0f)));
		//Style.Set("NoBorder", FAppStyle::Get().GetBrush("NoBorder"));
		Style.Set("NoBorder", new FSlateNoResource());
		Style.Set("ToolBar.Button.Normal", new FSlateNoResource());
		Style.Set("ToolBar.Button.Hovered", new BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, FLinearColor(0.1f, 0.1f, 0.1f)));

	}
	{
		const FSlateColor SelectionColor = FLinearColor(0.728f, 0.364f, 0.003f);
		const FSlateColor SelectionColor_Inactive = FLinearColor(0.25f, 0.25f, 0.25f);
		const FSlateColor SelectorColor = FLinearColor(0.701f, 0.225f, 0.003f);
		const FSlateColor DefaultForeground = FLinearColor(0.72f, 0.72f, 0.72f, 1.f);
		const FSlateColor InvertedForeground = FLinearColor(0, 0, 0);

		FTableRowStyle NormalTableRowStyle = FTableRowStyle()
			.SetEvenRowBackgroundBrush(FSlateNoResource())
			.SetEvenRowBackgroundHoveredBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, FLinearColor(1.0f, 1.0f, 1.0f, 0.1f)))
			.SetOddRowBackgroundBrush(FSlateNoResource())
			.SetOddRowBackgroundHoveredBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, FLinearColor(1.0f, 1.0f, 1.0f, 0.1f)))
			.SetSelectorFocusedBrush(BORDER_BRUSH("Common/Selector", FMargin(4.f / 16.f), SelectorColor))
			.SetActiveBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor))
			.SetActiveHoveredBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor))
			.SetInactiveBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor_Inactive))
			.SetInactiveHoveredBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor_Inactive))
			.SetTextColor(DefaultForeground)
			.SetSelectedTextColor(InvertedForeground)
			.SetDropIndicator_Above(BOX_BRUSH("Common/DropZoneIndicator_Above", FMargin(10.0f / 16.0f, 10.0f / 16.0f, 0, 0), SelectionColor))
			.SetDropIndicator_Onto(BOX_BRUSH("Common/DropZoneIndicator_Onto", FMargin(4.0f / 16.0f), SelectionColor))
			.SetDropIndicator_Below(BOX_BRUSH("Common/DropZoneIndicator_Below", FMargin(10.0f / 16.0f, 0, 0, 10.0f / 16.0f), SelectionColor));

		Style.Set("TableView.DarkRow", FTableRowStyle(NormalTableRowStyle)
			.SetEvenRowBackgroundBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, FLinearColor(1.0f, 1.0f, 1.0f, 0.1f)))
			.SetEvenRowBackgroundHoveredBrush(IMAGE_BRUSH("PropertyView/DetailCategoryMiddle_Hovered", FVector2f(16.f, 16.f)))
			.SetOddRowBackgroundBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, FLinearColor(1.0f, 1.0f, 1.0f, 0.1f)))
			.SetOddRowBackgroundHoveredBrush(IMAGE_BRUSH("PropertyView/DetailCategoryMiddle_Hovered", FVector2f(16.f, 16.f)))
			.SetSelectorFocusedBrush(BORDER_BRUSH("Common/Selector", FMargin(4.f / 16.f), SelectorColor))
			.SetActiveBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor))
			.SetActiveHoveredBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor))
			.SetInactiveBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor_Inactive))
			.SetInactiveHoveredBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor_Inactive))
			.SetTextColor(FLinearColor(1.0f, 1.0f, 1.0f, 0.1f))
			);
	}

	Style.Set("GenericFilters.TextStyle", FTextBlockStyle(NormalText)
		.SetFont(DEFAULT_FONT("Bold", 9))
		.SetColorAndOpacity(FLinearColor(1.0f, 1.0f, 1.0f, 0.9f))
		.SetShadowOffset(FVector2f(1.f, 1.f))
		.SetShadowColorAndOpacity(FLinearColor(0, 0, 0, 0.9f)));

	// Toolbar
	{
		Style.Set("Toolbar.Pause", new IMAGE_BRUSH("Icons/icon_pause_40x", Icon40x40));
		Style.Set("Toolbar.Resume", new IMAGE_BRUSH("Icons/icon_simulate_40x", Icon40x40));
		Style.Set("Toolbar.Record", new IMAGE_BRUSH("Icons/LV_Record", Icon40x40));
		Style.Set("Toolbar.Stop", new IMAGE_BRUSH("Icons/LV_Stop", Icon40x40));
		Style.Set("Toolbar.Camera", new IMAGE_BRUSH("Icons/LV_Camera", Icon40x40));
		Style.Set("Toolbar.Save", new IMAGE_BRUSH("Icons/LV_Save", Icon40x40));
		Style.Set("Toolbar.SaveAll", new IMAGE_BRUSH("Icons/icon_file_saveall_40x", Icon40x40));
		Style.Set("Toolbar.Load", new IMAGE_BRUSH("Icons/LV_Load", Icon40x40));
		Style.Set("Toolbar.Remove", new IMAGE_BRUSH("Icons/Edit/icon_Edit_Delete_40x", Icon40x40));
		Style.Set("Toolbar.Graphs", new IMAGE_BRUSH("Icons/icon_MatEd_Stats_40x", Icon40x40));
		Style.Set("Toolbar.HideEmpty", new IMAGE_BRUSH("Icons/icon_levels_visible_40x", Icon40x40));
		Style.Set("Toolbar.AutoScroll", new CORE_IMAGE_BRUSH_SVG("Starship/Insights/AutoScrollRight_20", CoreStyleConstants::Icon16x16));

		Style.Set("Toolbar.Pause.small", new IMAGE_BRUSH("Icons/icon_pause_40x", Icon20x20));
		Style.Set("Toolbar.Resume.small", new IMAGE_BRUSH("Icons/icon_simulate_40x", Icon20x20));
		Style.Set("Toolbar.Record.small", new IMAGE_BRUSH("Icons/LV_Record", Icon20x20));
		Style.Set("Toolbar.Stop.small", new IMAGE_BRUSH("Icons/LV_Stop", Icon20x20));
		Style.Set("Toolbar.Camera.small", new IMAGE_BRUSH("Icons/LV_Camera", Icon20x20));
		Style.Set("Toolbar.Save.small", new IMAGE_BRUSH("Icons/LV_Save", Icon20x20));
		Style.Set("Toolbar.SaveAll.small", new IMAGE_BRUSH("Icons/icon_file_saveall_40x", Icon20x20));
		Style.Set("Toolbar.Load.small", new IMAGE_BRUSH("Icons/LV_Load", Icon20x20));
		Style.Set("Toolbar.Remove.small", new IMAGE_BRUSH("Icons/Edit/icon_Edit_Delete_40x", Icon20x20));
		Style.Set("Toolbar.Graphs.small", new IMAGE_BRUSH("Icons/icon_MatEd_Stats_40x", Icon20x20));
		Style.Set("Toolbar.HideEmpty.small", new IMAGE_BRUSH("Icons/icon_levels_visible_40x", Icon20x20));
		Style.Set("Toolbar.AutoScroll,small", new CORE_IMAGE_BRUSH_SVG("Starship/Insights/AutoScrollRight_20", CoreStyleConstants::Icon16x16));
	}

	// Filters
	{
		Style.Set("Filters.FilterIcon", new CORE_IMAGE_BRUSH("Icons/Profiler/Profiler_Filter_Events_16x", Icon16x16));
		Style.Set("Filters.Style", FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("ToolbarComboButton"));
		Style.Set("ContentBrowser.FilterButtonBorder", new BOX_BRUSH("Common/RoundedSelection_16x", FMargin(4.0f / 16.0f)));
		//FSlateBrush *FilterButtonBorder = FAppStyle::Get().GetBrush("ToolbarComboButton");
		//Style.Set("ContentBrowser.FilterButtonBorder", FilterButtonBorder);
		
		//Style.Set("ContentBrowser.FilterButton", FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("ContentBrowser.FilterButton"));
		const FCheckBoxStyle ContentBrowserFilterButtonCheckBoxStyle = FCheckBoxStyle()
			.SetUncheckedImage(IMAGE_BRUSH("ContentBrowser/FilterUnchecked", FVector2f(10.0f, 14.0f)))
			.SetUncheckedHoveredImage(IMAGE_BRUSH("ContentBrowser/FilterUnchecked", FVector2f(10.0f, 14.0f), FLinearColor(0.5f, 0.5f, 0.5f, 1.0f)))
			.SetUncheckedPressedImage(IMAGE_BRUSH("ContentBrowser/FilterUnchecked", FVector2f(10.0f, 14.0f), FLinearColor(0.5f, 0.5f, 0.5f, 1.0f)))
			.SetCheckedImage(IMAGE_BRUSH("ContentBrowser/FilterChecked", FVector2f(10.0f, 14.0f)))
			.SetCheckedHoveredImage(IMAGE_BRUSH("ContentBrowser/FilterChecked", FVector2f(10.0f, 14.0f), FLinearColor(0.5f, 0.5f, 0.5f, 1.0f)))
			.SetCheckedPressedImage(IMAGE_BRUSH("ContentBrowser/FilterChecked", FVector2f(10.0f, 14.0f), FLinearColor(0.5f, 0.5f, 0.5f, 1.0f)));
		/* ... and add the new style */
		Style.Set("ContentBrowser.FilterButton", ContentBrowserFilterButtonCheckBoxStyle);
	}

	//Sequencer
	{
		Style.Set("Sequencer.ItemTitle.Normal", new BOX_BRUSH("Common/Button/simple_round_normal", FMargin(4 / 16.0f), FLinearColor(1, 1, 1, 1)));
		Style.Set("Sequencer.ItemTitle.Hover", new BOX_BRUSH("Common/Button/simple_round_hovered", FMargin(4 / 16.0f), FLinearColor(1, 1, 1, 1)));
		Style.Set("Sequencer.SectionArea.Background", new FSlateColorBrush(FColor::White));
		const FTextBlockStyle CLassNameTextStyle = FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Regular", 7));
		Style.Set("Sequencer.ClassNAme", CLassNameTextStyle);
	}

	// Default text styles
	{
		Style.Set("RichText.Background", new BOX_BRUSH("Common/FlatColorSquare", FVector2f(1.0f, 1.0f), FMargin(0), FLinearColor(FColor(0xffeff3f3))));

		Style.Set("RichText.RoundedBackground", new BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, FLinearColor(FColor(0xffeff3f3))));

		const FTextBlockStyle NormalRichTextStyle = FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Regular", 10))
			.SetColorAndOpacity(FLinearColor(FColor(0xff2c3e50)));

		Style.Set("RichText.Text", NormalRichTextStyle);
		Style.Set("RichText.TextHighlight", FTextBlockStyle(NormalRichTextStyle)
			.SetColorAndOpacity(FLinearColor(FColor(0xff19bc9c)))
			);

		Style.Set("RichText.TextBold", FTextBlockStyle(NormalRichTextStyle)
			.SetFont(DEFAULT_FONT("Bold", 10)));

		Style.Set("RichText.HeaderText2", FTextBlockStyle(NormalRichTextStyle)
			.SetFontSize(14));

		Style.Set("RichText.HeaderText1", FTextBlockStyle(NormalRichTextStyle)
			.SetFontSize(20));

		{
			const FButtonStyle RichTextHyperlinkButton = FButtonStyle()
				.SetNormal(BORDER_BRUSH("Old/HyperlinkDotted", FMargin(0, 0, 0, 3 / 16.0f), FLinearColor::Blue))
				.SetPressed(FSlateNoResource())
				.SetHovered(BORDER_BRUSH("Old/HyperlinkUnderline", FMargin(0, 0, 0, 3 / 16.0f), FLinearColor::Blue));

			const FTextBlockStyle RichTextHyperlinkText = FTextBlockStyle(NormalRichTextStyle)
				.SetColorAndOpacity(FLinearColor::Blue);

			Style.Set("RichText.HyperlinkText", RichTextHyperlinkText);

			// legacy style
			Style.Set("RichText.EditableHyperlinkText", RichTextHyperlinkText);

			const FHyperlinkStyle RichTextHyperlink = FHyperlinkStyle()
				.SetUnderlineStyle(RichTextHyperlinkButton)
				.SetTextStyle(RichTextHyperlinkText)
				.SetPadding(FMargin(0.0f));
			Style.Set("RichText.Hyperlink", RichTextHyperlink);

			Style.Set("RichText.ExternalLink", new IMAGE_BRUSH("Tutorials/ExternalLink", Icon16x16, FLinearColor::Blue));

			// legacy style
			Style.Set("RichText.EditableHyperlink", RichTextHyperlink);
		}

		//Tagline
		{
			Style.Set("RichText.Tagline.Background", new BOX_BRUSH("Common/FlatColorSquare", FVector2f(1.0f, 1.0f), FMargin(1), FLinearColor(FColor(0xffdbe4e4))));
			Style.Set("RichText.Tagline.DarkBackground", new BOX_BRUSH("Common/FlatColorSquare", FVector2f(1.0f, 1.0f), FMargin(1), FLinearColor(0.55423f, 0.60548f, 0.60548f)));
			Style.Set("RichText.Tagline.Text", FTextBlockStyle(NormalText)
				.SetFont(DEFAULT_FONT("Bold", 24))
				.SetColorAndOpacity(FLinearColor(FColor(0xff2c3e50)))
				);

			Style.Set("RichText.Tagline.TextHighlight", FTextBlockStyle(NormalText)
				.SetFont(DEFAULT_FONT("Bold", 24))
				.SetColorAndOpacity(FLinearColor(FColor(0xff19bc9c)))
				);

			Style.Set("RichText.Tagline.SubtleText", FTextBlockStyle(NormalText)
				.SetFont(DEFAULT_FONT("Regular", 16))
				.SetColorAndOpacity(FLinearColor(FColor(0xff2c3e50)))
				);

			Style.Set("RichText.Tagline.SubtleTextHighlight", FTextBlockStyle(NormalText)
				.SetFont(DEFAULT_FONT("Regular", 16))
				.SetColorAndOpacity(FLinearColor(FColor(0xff19bc9c)))
				);
		}

	}

	return StyleRef;
}

#undef IMAGE_BRUSH
#undef BOX_BRUSH
#undef BORDER_BRUSH

const ISlateStyle& FLogVisualizerStyle::Get()
{
	return *StyleInstance;
}
