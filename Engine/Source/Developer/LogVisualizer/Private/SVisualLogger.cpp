// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVisualLogger.h"

#if UE_DEBUG_VISUALIZER_TOOL_ENABLED
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/FileManager.h"
#include "Misc/CommandLine.h"
#include "Modules/ModuleManager.h"
#include "Async/Future.h"
#include "Async/Async.h"
#include "Debug/DebugDrawService.h"
#include "AI/NavigationSystemBase.h"
#include "Engine/Engine.h"
#include "EngineUtils.h"
#include "VisualLogger/VisualLogger.h"
#include "LogVisualizerSettings.h"
#include "LogVisualizerSessionSettings.h"
#include "VisualLoggerDatabase.h"
#include "LogVisualizerStyle.h"
#include "VisualLoggerCommands.h"
#include "Widgets/Docking/SDockTab.h"
#include "SVisualLoggerToolbar.h"
#include "SVisualLoggerFilters.h"
#include "SVisualLoggerView.h"
#include "SVisualLoggerLogsList.h"
#include "SVisualLoggerStatusView.h"
#include "SVisualLoggerTab.h"
#include "VisualLoggerTimeSliderController.h"
#include "VisualLoggerRenderingActor.h"
#include "VisualLoggerCanvasRenderer.h"
#include "DesktopPlatformModule.h"
#include "VisualLoggerCameraController.h"
#include "Framework/Application/SlateApplication.h"
#if WITH_EDITOR
#include "Editor/EditorEngine.h"
#include "Editor.h"
#endif
#include "GameDelegates.h"
#include "ISettingsModule.h"

#include "VisualLogger/VisualLoggerBinaryFileDevice.h"
#include "VisualLogger/VisualLoggerFilterVolume.h"

#define LOCTEXT_NAMESPACE "SVisualLogger"

DEFINE_LOG_CATEGORY_STATIC(LogVisualLogger, Log, All);

/* Local constants
*****************************************************************************/
static const FName ToolbarTabId("Toolbar");
static const FName FiltersTabId("Filters");
static const FName MainViewTabId("MainView");
static const FName LogsListTabId("LogsList");
static const FName StatusViewTabId("StatusView");

namespace LogVisualizer
{
	static const FString LogFileDescription = LOCTEXT("FileTypeDescription", "Visual Log File").ToString();
	static const FString LoadFileTypes = FString::Printf(TEXT("%s (*.bvlog;*.%s)|*.bvlog;*.%s"), *LogFileDescription, VISLOG_FILENAME_EXT, VISLOG_FILENAME_EXT);
	static const FString SaveFileTypes = FString::Printf(TEXT("%s (*.%s)|*.%s"), *LogFileDescription, VISLOG_FILENAME_EXT, VISLOG_FILENAME_EXT);
}

/* SMessagingDebugger constructors
*****************************************************************************/
namespace
{
	static UWorld* GetWorldForGivenObject(const UObject* Object)
	{
		UWorld* World = GEngine->GetWorldFromContextObject(Object, EGetWorldErrorMode::ReturnNull);
#if WITH_EDITOR
		UEditorEngine* EEngine = Cast<UEditorEngine>(GEngine);
		if (GIsEditor && EEngine != nullptr && World == nullptr)
		{
			// lets use PlayWorld during PIE/Simulate and regular world from editor otherwise, to draw debug information
			World = EEngine->PlayWorld != nullptr ? ToRawPtr(EEngine->PlayWorld) : EEngine->GetEditorWorldContext().World();
		}

#endif
		if (!GIsEditor && World == nullptr)
		{
			World = GEngine->GetWorld();
		}

		return World;
	}
}

SVisualLogger::SVisualLogger() 
	: SCompoundWidget(), CommandList(MakeShareable(new FUICommandList))
{ 
	bPausedLogger = false;
	bGotHistogramData = false;
	bAutoScrollToLastItem = false;
	bAutoScrollingToLastItem = false;

	class FVisualLoggerDevice : public FVisualLogDevice
	{
	public:
		FVisualLoggerDevice(SVisualLogger& InVisualLogger) : VisualLoggerWidget(InVisualLogger) {}

		virtual ~FVisualLoggerDevice(){}
		virtual void Serialize(const UObject* InLogOwner, const FName& InOwnerName, const FName& InOwnerDisplayName, const FName& InOwnerClassName, const FVisualLogEntry& InLogEntry) override
		{
			VisualLoggerWidget.OnNewLogEntry(FVisualLogDevice::FVisualLogEntryItem(InOwnerName, InOwnerDisplayName, InOwnerClassName, InLogEntry));
		}
		
		SVisualLogger& VisualLoggerWidget;
	};

	InternalDevice = MakeShareable(new FVisualLoggerDevice(*this));
	FVisualLogger::Get().AddDevice(InternalDevice.Get());
}

SVisualLogger::~SVisualLogger()
{
#if WITH_EDITOR
	FWorldDelegates::OnPostWorldInitialization.Remove(PostWorldInitializationHandle);
	FGameDelegates::Get().GetEndPlayMapDelegate().RemoveAll(this);
#endif
		
	TabManager->CloseAllAreas();
	TabManager->UnregisterAllTabSpawners();

	GEngine->OnWorldAdded().RemoveAll(this);

	FVisualLogger::Get().RemoveDevice(InternalDevice.Get());
	InternalDevice.Reset();

#if WITH_EDITOR
	GetMutableDefault<ULogVisualizerSettings>()->SavePersistentData();
	GetMutableDefault<ULogVisualizerSettings>()->OnSettingChanged().RemoveAll(this);
#endif

	if (LastUsedWorld.IsValid())
	{
		for (TActorIterator<AVisualLoggerRenderingActor> It(LastUsedWorld.Get()); It; ++It)
		{
			LastUsedWorld->DestroyActor(*It);
		}
	}


	UDebugDrawService::Unregister(DrawOnCanvasDelegateHandle);
	VisualLoggerCanvasRenderer.Reset();

	FVisualLoggerDatabase::Get().GetEvents().OnRowSelectionChanged.RemoveAll(this);
	FVisualLoggerDatabase::Get().GetEvents().OnNewItem.RemoveAll(this);
	FVisualLoggerDatabase::Get().GetEvents().OnItemSelectionChanged.RemoveAll(this);

	FLogVisualizer::Get().GetEvents().OnFiltersChanged.RemoveAll(this);
	FLogVisualizer::Get().GetEvents().OnLogLineSelectionChanged.Unbind();
	FLogVisualizer::Get().GetEvents().OnKeyboardEvent.Unbind();

	GEngine->OnLevelActorAdded().RemoveAll(this);
	GEngine->OnLevelActorDeleted().RemoveAll(this);
	GEngine->OnActorMoved().RemoveAll(this);

	FVisualLoggerDatabase::Get().Reset();
}

/* SMessagingDebugger interface
*****************************************************************************/

void SVisualLogger::Construct(const FArguments& InArgs, const TSharedRef<SDockTab>& ConstructUnderMajorTab, const TSharedPtr<SWindow>& ConstructUnderWindow)
{
	bPausedLogger = false;
	bGotHistogramData = false;
	bAutoScrollToLastItem = false;
	bAutoScrollingToLastItem = false;

	FLogVisualizer::Get().SetCurrentVisualizer(SharedThis(this));
	//////////////////////////////////////////////////////////////////////////
	// Visual Logger Events
	FLogVisualizer::Get().GetEvents().OnFiltersChanged.AddRaw(this, &SVisualLogger::OnFiltersChanged);
	FLogVisualizer::Get().GetEvents().OnLogLineSelectionChanged = FOnLogLineSelectionChanged::CreateSP(this, &SVisualLogger::OnLogLineSelectionChanged);
	FLogVisualizer::Get().GetEvents().OnKeyboardEvent = FOnKeyboardEvent::CreateSP(this, &SVisualLogger::OnKeyboardRedirection);
	FLogVisualizer::Get().GetTimeSliderController().Get()->GetTimeSliderArgs().OnScrubPositionChanged = FVisualLoggerTimeSliderArgs::FOnScrubPositionChanged::CreateSP(this, &SVisualLogger::OnScrubPositionChanged);

	FVisualLoggerDatabase::Get().GetEvents().OnRowSelectionChanged.AddRaw(this, &SVisualLogger::OnObjectSelectionChanged);
	FVisualLoggerDatabase::Get().GetEvents().OnNewItem.AddRaw(this, &SVisualLogger::OnNewItemHandler);
	FVisualLoggerDatabase::Get().GetEvents().OnItemSelectionChanged.AddRaw(this, &SVisualLogger::OnItemsSelectionChanged);

	LastUsedWorld = FVisualLoggerEditorInterface::Get()->GetWorld();
	CollectFilterVolumes();
	ProcessFilterVolumes();

#if WITH_EDITOR
	PostWorldInitializationHandle = FWorldDelegates::OnPostWorldInitialization.AddLambda([this](UWorld*, const UWorld::InitializationValues)
	{
		UWorld* World = FVisualLoggerEditorInterface::Get()->GetWorld();
		if (World != nullptr && World != LastUsedWorld)
		{
			OnNewWorld(World);
		}
	});

	// Note that we use EndPlay delegate instead of FEditorDelegates::EndPie since we want 
	// the teardown of the PlayWorld to be completed so FVisualLoggerEditorInterface will return
	// us the Editor world (if any) as the new world.
	FGameDelegates::Get().GetEndPlayMapDelegate().AddLambda([this]()
	{
		UWorld* World = FVisualLoggerEditorInterface::Get()->GetWorld();
		if (World != nullptr && World != LastUsedWorld)
		{
			OnNewWorld(World);
		}
	});

	GetMutableDefault<ULogVisualizerSettings>()->OnSettingChanged().AddRaw(this, &SVisualLogger::OnSettingsChanged);
#endif

	GEngine->OnWorldAdded().AddRaw(this, &SVisualLogger::OnNewWorld);

	GEngine->OnLevelActorAdded().AddRaw(this, &SVisualLogger::OnLevelActorAdded);
	GEngine->OnLevelActorDeleted().AddRaw(this, &SVisualLogger::OnLevelActorDeleted);
	GEngine->OnActorMoved().AddRaw(this, &SVisualLogger::OnActorMoved);

	//////////////////////////////////////////////////////////////////////////
	// Command Action Lists
	const FVisualLoggerCommands& Commands = FVisualLoggerCommands::Get();
	FUICommandList& ActionList = *CommandList;

	ULogVisualizerSettings* Settings = GetMutableDefault<ULogVisualizerSettings>();
	Settings->ConfigureVisLog();
	Settings->LoadPersistentData();

	ActionList.MapAction(Commands.StartRecording, FExecuteAction::CreateSP(this, &SVisualLogger::HandleStartRecordingCommandExecute), FCanExecuteAction::CreateSP(this, &SVisualLogger::HandleStartRecordingCommandCanExecute), FIsActionChecked(), FIsActionButtonVisible::CreateSP(this, &SVisualLogger::HandleStartRecordingCommandIsVisible));
	ActionList.MapAction(Commands.StopRecording, FExecuteAction::CreateSP(this, &SVisualLogger::HandleStopRecordingCommandExecute), FCanExecuteAction::CreateSP(this, &SVisualLogger::HandleStopRecordingCommandCanExecute), FIsActionChecked(), FIsActionButtonVisible::CreateSP(this, &SVisualLogger::HandleStopRecordingCommandIsVisible));
	ActionList.MapAction(Commands.Pause, FExecuteAction::CreateSP(this, &SVisualLogger::HandlePauseCommandExecute), FCanExecuteAction::CreateSP(this, &SVisualLogger::HandlePauseCommandCanExecute), FIsActionChecked(), FIsActionButtonVisible::CreateSP(this, &SVisualLogger::HandlePauseCommandIsVisible));
	ActionList.MapAction(Commands.Resume, FExecuteAction::CreateSP(this, &SVisualLogger::HandleResumeCommandExecute), FCanExecuteAction::CreateSP(this, &SVisualLogger::HandleResumeCommandCanExecute), FIsActionChecked(), FIsActionButtonVisible::CreateSP(this, &SVisualLogger::HandleResumeCommandIsVisible));
	ActionList.MapAction(Commands.LoadFromVLog, FExecuteAction::CreateSP(this, &SVisualLogger::HandleLoadCommandExecute), FCanExecuteAction::CreateSP(this, &SVisualLogger::HandleLoadCommandCanExecute), FIsActionChecked(), FIsActionButtonVisible::CreateSP(this, &SVisualLogger::HandleLoadCommandCanExecute));
	ActionList.MapAction(Commands.SaveToVLog, FExecuteAction::CreateSP(this, &SVisualLogger::HandleSaveCommandExecute), FCanExecuteAction::CreateSP(this, &SVisualLogger::HandleSaveCommandCanExecute), FIsActionChecked(), FIsActionButtonVisible::CreateSP(this, &SVisualLogger::HandleSaveCommandCanExecute));
	ActionList.MapAction(Commands.SaveAllToVLog, FExecuteAction::CreateSP(this, &SVisualLogger::HandleSaveAllCommandExecute), FCanExecuteAction::CreateSP(this, &SVisualLogger::HandleSaveCommandCanExecute), FIsActionChecked(), FIsActionButtonVisible::CreateSP(this, &SVisualLogger::HandleSaveCommandCanExecute));
	ActionList.MapAction(Commands.FreeCamera,
		FExecuteAction::CreateSP(this, &SVisualLogger::HandleCameraCommandExecute), 
		FCanExecuteAction::CreateSP(this, &SVisualLogger::HandleCameraCommandCanExecute), 
		FIsActionChecked::CreateSP(this, &SVisualLogger::HandleCameraCommandIsChecked),
		FIsActionButtonVisible::CreateSP(this, &SVisualLogger::HandleCameraCommandCanExecute));
	ActionList.MapAction(Commands.ToggleGraphs,
		FExecuteAction::CreateLambda([](){bool& bEnableGraphsVisualization = ULogVisualizerSessionSettings::StaticClass()->GetDefaultObject<ULogVisualizerSessionSettings>()->bEnableGraphsVisualization; bEnableGraphsVisualization = !bEnableGraphsVisualization; }),
		FCanExecuteAction::CreateLambda([this]()->bool{return FVisualLoggerGraphsDatabase::Get().ContainsHistogramGraphs(); }),
		FIsActionChecked::CreateLambda([]()->bool{return ULogVisualizerSessionSettings::StaticClass()->GetDefaultObject<ULogVisualizerSessionSettings>()->bEnableGraphsVisualization; }),
		FIsActionButtonVisible::CreateLambda([this]()->bool{return FVisualLoggerGraphsDatabase::Get().ContainsHistogramGraphs(); }));
	ActionList.MapAction(Commands.ResetData, 
		FExecuteAction::CreateSP(this, &SVisualLogger::ResetData),
		FCanExecuteAction::CreateSP(this, &SVisualLogger::HandleSaveCommandCanExecute),
		FIsActionChecked(), 
		FIsActionButtonVisible::CreateSP(this, &SVisualLogger::HandleSaveCommandCanExecute));
	ActionList.MapAction(Commands.Refresh,
		FExecuteAction::CreateSP(this, &SVisualLogger::HandleRefreshCommandExecute),
		FCanExecuteAction::CreateSP(this, &SVisualLogger::HandleRefreshCommandCanExecute),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateSP(this, &SVisualLogger::HandleRefreshCommandCanExecute));
	ActionList.MapAction(Commands.AutoScroll,
		FExecuteAction::CreateSP(this, &SVisualLogger::HandleAutoScrollCommandExecute),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SVisualLogger::HandleAutoScrollIsChecked),
		FIsActionButtonVisible());


	// Tab Spawners
	TabManager = FGlobalTabmanager::Get()->NewTabManager(ConstructUnderMajorTab);
	TSharedRef<FWorkspaceItem> AppMenuGroup = TabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("VisualLoggerGroupName", "Visual Logger"));

	TabManager->RegisterTabSpawner(ToolbarTabId, FOnSpawnTab::CreateSP(this, &SVisualLogger::HandleTabManagerSpawnTab, ToolbarTabId))
		.SetDisplayName(LOCTEXT("ToolbarTabTitle", "Toolbar"))
		.SetGroup(AppMenuGroup)
		.SetIcon(FSlateIcon(FLogVisualizerStyle::Get().GetStyleSetName(), "ToolbarTabIcon"));

	TabManager->RegisterTabSpawner(FiltersTabId, FOnSpawnTab::CreateSP(this, &SVisualLogger::HandleTabManagerSpawnTab, FiltersTabId))
		.SetDisplayName(LOCTEXT("FiltersTabTitle", "Filters"))
		.SetGroup(AppMenuGroup)
		.SetIcon(FSlateIcon(FLogVisualizerStyle::Get().GetStyleSetName(), "FiltersTabIcon"));

	TabManager->RegisterTabSpawner(MainViewTabId, FOnSpawnTab::CreateSP(this, &SVisualLogger::HandleTabManagerSpawnTab, MainViewTabId))
		.SetDisplayName(LOCTEXT("MainViewTabTitle", "MainView"))
		.SetGroup(AppMenuGroup)
		.SetIcon(FSlateIcon(FLogVisualizerStyle::Get().GetStyleSetName(), "MainViewTabIcon"));

	TabManager->RegisterTabSpawner(LogsListTabId, FOnSpawnTab::CreateSP(this, &SVisualLogger::HandleTabManagerSpawnTab, LogsListTabId))
		.SetDisplayName(LOCTEXT("LogsListTabTitle", "LogsList"))
		.SetGroup(AppMenuGroup)
		.SetIcon(FSlateIcon(FLogVisualizerStyle::Get().GetStyleSetName(), "LogsListTabIcon"));

	TabManager->RegisterTabSpawner(StatusViewTabId, FOnSpawnTab::CreateSP(this, &SVisualLogger::HandleTabManagerSpawnTab, StatusViewTabId))
		.SetDisplayName(LOCTEXT("StatusViewTabTitle", "StatusView"))
		.SetGroup(AppMenuGroup)
		.SetIcon(FSlateIcon(FLogVisualizerStyle::Get().GetStyleSetName(), "StatusViewTabIcon"));

	// Default Layout
	const TSharedRef<FTabManager::FLayout> Layout = FTabManager::NewLayout("VisualLoggerLayout_v1.0")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewStack()
				->AddTab(ToolbarTabId, ETabState::OpenedTab)
				->SetHideTabWell(true)
			)
			->Split
			(
				FTabManager::NewStack()
				->AddTab(FiltersTabId, ETabState::OpenedTab)
				->SetHideTabWell(true)
			)
			->Split
			(
				FTabManager::NewStack()
				->AddTab(MainViewTabId, ETabState::OpenedTab)
				->SetHideTabWell(true)
			)
			->Split
			(
				FTabManager::NewSplitter()
				->SetOrientation(Orient_Horizontal)
				->SetSizeCoefficient(0.6f)
				->Split
				(
					FTabManager::NewStack()
					->AddTab(StatusViewTabId, ETabState::OpenedTab)
					->SetHideTabWell(true)
					->SetSizeCoefficient(0.3f)
					)
				->Split
				(
					FTabManager::NewStack()
					->AddTab(LogsListTabId, ETabState::OpenedTab)
					->SetHideTabWell(true)
					->SetSizeCoefficient(0.7f)
					)
			)
		);
	TabManager->SetOnPersistLayout(FTabManager::FOnPersistLayout::CreateSP(this, &SVisualLogger::HandleTabManagerPersistLayout));

	// Window Menu
	FMenuBarBuilder MenuBarBuilder = FMenuBarBuilder(TSharedPtr<FUICommandList>());
	MenuBarBuilder.AddPullDownMenu(
		LOCTEXT("WindowMenuLabel", "Window"),
		FText::GetEmpty(),
		FNewMenuDelegate::CreateStatic(&SVisualLogger::FillWindowMenu, TabManager),
		"Window"
		);

	MenuBarBuilder.AddMenuEntry(
		LOCTEXT("SettingsMenuLabel", "Settings"),
		FText::GetEmpty(),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda(
			[this](){
				ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
				if (SettingsModule != nullptr)
				{
					SettingsModule->ShowViewer("Editor", "Advanced", "VisualLogger");
				}
			}
		)),
		"Settings"
		);


	ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				MenuBarBuilder.MakeWidget()
			]
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				TabManager->RestoreFrom(Layout, ConstructUnderWindow).ToSharedRef()
			]
		];

	VisualLoggerCanvasRenderer = MakeShareable(new FVisualLoggerCanvasRenderer());

	DrawOnCanvasDelegateHandle = UDebugDrawService::Register(TEXT("VisLog"), FDebugDrawDelegate::CreateRaw(VisualLoggerCanvasRenderer.Get(), &FVisualLoggerCanvasRenderer::DrawOnCanvas));

	// Call GetHelperActor to create the helper actor
	FVisualLoggerEditorInterface::Get()->GetHelperActor(LastUsedWorld.Get());
}

void SVisualLogger::OnNewLogEntry(const FVisualLogDevice::FVisualLogEntryItem& Entry)
{
	if (bPausedLogger)
	{
		OnPauseCacheForEntries.Add(Entry);
		return;
	}

	FVisualLoggerDatabase::Get().AddItem(Entry);
}

void SVisualLogger::HandleMajorTabPersistVisualState()
{
	// save any settings here
}

void SVisualLogger::HandleTabManagerPersistLayout(const TSharedRef<FTabManager::FLayout>& LayoutToSave)
{
	// save any layout here
}

void SVisualLogger::FillWindowMenu(FMenuBuilder& MenuBuilder, const TSharedPtr<FTabManager> TabManager)
{
	if (!TabManager.IsValid())
	{
		return;
	}

	TabManager->PopulateLocalTabSpawnerMenu(MenuBuilder);
}

TSharedRef<SDockTab> SVisualLogger::HandleTabManagerSpawnTab(const FSpawnTabArgs& Args, FName TabIdentifier) const
{
	TSharedPtr<SWidget> TabWidget = SNullWidget::NullWidget;
	bool AutoSizeTab = false;

	if (TabIdentifier == ToolbarTabId)
	{ 
		TabWidget = SNew(SVisualLoggerToolbar, CommandList);
		AutoSizeTab = true;
	}
	else if (TabIdentifier == FiltersTabId)
	{
		TabWidget = SAssignNew(VisualLoggerFilters, SVisualLoggerFilters, CommandList);
		AutoSizeTab = true;
	}
	else if (TabIdentifier == MainViewTabId)
	{
		TabWidget = SAssignNew(MainView, SVisualLoggerView, CommandList).OnFiltersSearchChanged(const_cast<SVisualLogger*>(this), &SVisualLogger::OnFiltersSearchChanged);
		AutoSizeTab = false;
	}
	else if (TabIdentifier == LogsListTabId)
	{
		TabWidget = SAssignNew(LogsList, SVisualLoggerLogsList, CommandList);
		AutoSizeTab = false;
	}
	else if (TabIdentifier == StatusViewTabId)
	{
		TabWidget = SAssignNew(StatusView, SVisualLoggerStatusView, CommandList);
		AutoSizeTab = false;
	}

	check(TabWidget.IsValid());
	return SNew(SVisualLoggerTab)
		.ShouldAutosize(AutoSizeTab)
		.TabRole(ETabRole::DocumentTab)
		[
			TabWidget.ToSharedRef()
		];
}

bool SVisualLogger::HandleStartRecordingCommandCanExecute() const
{
	return !FVisualLogger::Get().IsRecording();
}


void SVisualLogger::HandleStartRecordingCommandExecute()
{
	FVisualLogger::Get().SetIsRecording(true);
}


bool SVisualLogger::HandleStartRecordingCommandIsVisible() const
{
	return !FVisualLogger::Get().IsRecording();
}

bool SVisualLogger::HandleStopRecordingCommandCanExecute() const
{
	return FVisualLogger::Get().IsRecording();
}


void SVisualLogger::HandleStopRecordingCommandExecute()
{
	UWorld* World = FLogVisualizer::Get().GetWorld();

#if UE_DEBUG_RECORDING_USING_VLOG
	if (FParse::Param(FCommandLine::Get(), TEXT("LogNavOctree")) == true && GetDefault<ULogVisualizerSettings>()->bLogNavOctreeOnStop)
	{
		FVisualLogger::NavigationDataDump(World, LogNavigation, ELogVerbosity::Log, FBox());
	}
#endif // UE_DEBUG_RECORDING_USING_VLOG

	FVisualLogger::Get().SetIsRecording(false);

	if (AVisualLoggerCameraController::IsEnabled(World))
	{
		AVisualLoggerCameraController::DisableCamera(World);
	}

	if (bPausedLogger)
	{
		HandleResumeCommandExecute();
	}
}


bool SVisualLogger::HandleStopRecordingCommandIsVisible() const
{
	return FVisualLogger::Get().IsRecording();
}

bool SVisualLogger::HandlePauseCommandCanExecute() const
{
	return !bPausedLogger;
}

void SVisualLogger::HandlePauseCommandExecute()
{
	if (GetDefault<ULogVisualizerSettings>()->bUsePlayersOnlyForPause)
	{
		const TIndirectArray<FWorldContext>& WorldContexts = GEngine->GetWorldContexts();
		for (const FWorldContext& Context : WorldContexts)
		{
			if (Context.World() != nullptr)
			{
				Context.World()->bPlayersOnlyPending = true;
			}
		}
	}

	bPausedLogger = true;
}

bool SVisualLogger::HandlePauseCommandIsVisible() const
{
	return HandlePauseCommandCanExecute();
}

bool SVisualLogger::HandleResumeCommandCanExecute() const
{
	return bPausedLogger;
}

void SVisualLogger::HandleResumeCommandExecute()
{
	if (GetDefault<ULogVisualizerSettings>()->bUsePlayersOnlyForPause)
	{
		const TIndirectArray<FWorldContext>& WorldContexts = GEngine->GetWorldContexts();
		for (const FWorldContext& Context : WorldContexts)
		{
			if (Context.World() != nullptr)
			{
				Context.World()->bPlayersOnly = false;
				Context.World()->bPlayersOnlyPending = false;
			}
		}
	}

	bPausedLogger = false;
	for (const auto& CurrentEntry : OnPauseCacheForEntries)
	{
		OnNewLogEntry(CurrentEntry);
	}
	OnPauseCacheForEntries.Reset();

}

bool SVisualLogger::HandleResumeCommandIsVisible() const
{
	return HandleResumeCommandCanExecute();
}

bool SVisualLogger::HandleCameraCommandIsChecked() const
{
	UWorld* World = FLogVisualizer::Get().GetWorld();
	return World && AVisualLoggerCameraController::IsEnabled(World);
}

bool SVisualLogger::HandleCameraCommandCanExecute() const
{
	UWorld* World = FLogVisualizer::Get().GetWorld();
	return FVisualLogger::Get().IsRecording() && World && (World->bPlayersOnly || World->bPlayersOnlyPending) && World->IsPlayInEditor() && (GEditor && !GEditor->bIsSimulatingInEditor);
}

void SVisualLogger::HandleCameraCommandExecute()
{
	UWorld* World = FLogVisualizer::Get().GetWorld();
	if (AVisualLoggerCameraController::IsEnabled(World))
	{
		AVisualLoggerCameraController::DisableCamera(World);
	}
	else
	{
		// switch debug cam on
		CameraController = AVisualLoggerCameraController::EnableCamera(World);
	}
}

bool SVisualLogger::HandleRefreshCommandCanExecute() const
{
	UWorld* World = FLogVisualizer::Get().GetWorld();
	return FVisualLogger::Get().IsRecording() && World && World->IsEditorWorld();
}

void SVisualLogger::HandleRefreshCommandExecute()
{
	FVisualLogger::Get().Flush();
}

bool SVisualLogger::HandleLoadCommandCanExecute() const
{
	return true;
}

void SVisualLogger::HandleAutoScrollCommandExecute()
{
	bAutoScrollToLastItem = !bAutoScrollToLastItem;
	if (bAutoScrollToLastItem)
	{
		TGuardValue<bool> Guard(bAutoScrollingToLastItem, true);
		FLogVisualizer::Get().GotoLastItemAnyRow();
	}
}

bool SVisualLogger::HandleAutoScrollIsChecked() const
{
	return bAutoScrollToLastItem;
}

void SVisualLogger::HandleLoadCommandExecute()
{
	FArchive Ar;
	TArray<FVisualLogDevice::FVisualLogEntryItem> RecordedLogs;

	TArray<FString> OpenFilenames;
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	bool bOpened = false;
	if (DesktopPlatform)
	{
		const FString DefaultBrowsePath = FString::Printf(TEXT("%slogs/"), *FPaths::ProjectSavedDir());

		bOpened = DesktopPlatform->OpenFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(AsShared()),
			LOCTEXT("OpenProjectBrowseTitle", "Open Project").ToString(),
			DefaultBrowsePath,
			TEXT(""),
			LogVisualizer::LoadFileTypes,
			EFileDialogFlags::None,
			OpenFilenames
			);
	}

	if (bOpened && OpenFilenames.Num() > 0)
	{
		OnNewWorld(GetWorldForGivenObject(nullptr));
		for (int FilenameIndex = 0; FilenameIndex < OpenFilenames.Num(); ++FilenameIndex)
		{
			FString CurrentFileName = OpenFilenames[FilenameIndex];
			const bool bIsBinaryFile = CurrentFileName.Find(TEXT(".bvlog")) != INDEX_NONE;
			if (bIsBinaryFile)
			{
				FArchive* FileAr = IFileManager::Get().CreateFileReader(*CurrentFileName);
				FVisualLoggerHelpers::Serialize(*FileAr, RecordedLogs);
				FileAr->Close();
				delete FileAr;
				FileAr = NULL;

				for (FVisualLogDevice::FVisualLogEntryItem& CurrentItem : RecordedLogs)
				{
					OnNewLogEntry(CurrentItem);
				}
			}
		}
	}
}

bool SVisualLogger::HandleSaveCommandCanExecute() const
{ 
	return FVisualLoggerDatabase::Get().NumberOfRows() > 0;
}

void SVisualLogger::HandleSaveAllCommandExecute()
{
	HandleSaveCommand(true);
}

void SVisualLogger::HandleSaveCommandExecute()
{
	HandleSaveCommand(false);
}

void SVisualLogger::HandleSaveCommand(bool bSaveAllData)
{
	TArray<FName> SelectedRows;
	if (!bSaveAllData)
	{
		SelectedRows = FVisualLoggerDatabase::Get().GetSelectedRows();
	}
	else
	{
		for (auto Iter(FVisualLoggerDatabase::Get().GetConstRowIterator()); Iter; ++Iter)
		{
			SelectedRows.Add((*Iter).GetOwnerName());
		}
	}

	if (SelectedRows.Num())
	{
		// Prompt the user for the filenames
		TArray<FString> SaveFilenames;
		IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
		bool bSaved = false;
		if (DesktopPlatform)
		{
			const FString DefaultBrowsePath = FString::Printf(TEXT("%slogs/"), *FPaths::ProjectSavedDir());
			bSaved = DesktopPlatform->SaveFileDialog(
				FSlateApplication::Get().FindBestParentWindowHandleForDialogs(AsShared()),
				LOCTEXT("NewProjectBrowseTitle", "Choose a project location").ToString(),
				DefaultBrowsePath,
				TEXT(""),
				LogVisualizer::SaveFileTypes,
				EFileDialogFlags::None,
				SaveFilenames
				);
		}

		if (bSaved)
		{
			if (SaveFilenames.Num() > 0 && SaveFilenames[0].IsEmpty() == false)
			{
				TArray<FVisualLogDevice::FVisualLogEntryItem> FrameCache;
				for (const FName& CurrentName : SelectedRows)
				{
					FVisualLoggerDBRow& DataRow = FVisualLoggerDatabase::Get().GetRowByName(CurrentName);
					FrameCache.Append(DataRow.GetItems());
				}

				if (FrameCache.Num())
				{
					FArchive* FileArchive = IFileManager::Get().CreateFileWriter(*SaveFilenames[0]);
					if (ensure(FileArchive))
					{
						FVisualLoggerHelpers::Serialize(*FileArchive, FrameCache);
						FileArchive->Close();
						delete FileArchive;
						FileArchive = NULL;
					}
					else
					{
						UE_LOGF(LogVisualLogger, Error, "Failed to create file \"%ls\"", *SaveFilenames[0]);
					}
				}
			}
		}
	}
}

void SVisualLogger::ResetData()
{
	bGotHistogramData = false;
	OnPauseCacheForEntries.Reset();

	FLogVisualizer::Get().Reset();
	FVisualLoggerDatabase::Get().Reset();

	FVisualLoggerFilters::Get().Reset();

	if (MainView.IsValid())
	{
		MainView->ResetData();
	}

	if (LogsList.IsValid())
	{
		LogsList->ResetData();
	}

	if (StatusView.IsValid())
	{
		StatusView->ResetData();
	}

	if (VisualLoggerCanvasRenderer.IsValid())
	{
		VisualLoggerCanvasRenderer->ResetData();
	}

	if (AVisualLoggerRenderingActor* HelperActor = Cast<AVisualLoggerRenderingActor>(FVisualLoggerEditorInterface::Get()->GetHelperActor(LastUsedWorld.Get())))
	{
		HelperActor->ResetRendering();
	}

	const TMap<FName, FVisualLogExtensionInterface*>& AllExtensions = FVisualLogger::Get().GetAllExtensions();
	for (auto Iterator = AllExtensions.CreateConstIterator(); Iterator; ++Iterator)
	{
		FVisualLogExtensionInterface* Extension = (*Iterator).Value;
		if (Extension != NULL)
		{
			Extension->ResetData(FVisualLoggerEditorInterface::Get());
		}
	}

	FLogVisualizer::Get().GetEvents().OnLogLineSelectionChanged = FOnLogLineSelectionChanged::CreateSP(this, &SVisualLogger::OnLogLineSelectionChanged);
	FLogVisualizer::Get().GetEvents().OnKeyboardEvent = FOnKeyboardEvent::CreateSP(this, &SVisualLogger::OnKeyboardRedirection);
	FLogVisualizer::Get().GetTimeSliderController().Get()->GetTimeSliderArgs().OnScrubPositionChanged = FVisualLoggerTimeSliderArgs::FOnScrubPositionChanged::CreateSP(this, &SVisualLogger::OnScrubPositionChanged);
}

void SVisualLogger::CollectFilterVolumes()
{
	FilterVolumesInLastUsedWorld.Reset();
	for (TActorIterator<AActor> It(LastUsedWorld.Get(), AVisualLoggerFilterVolume::StaticClass()); It; ++It)
	{
		AVisualLoggerFilterVolume* Volume = Cast<AVisualLoggerFilterVolume>(*It);
		FilterVolumesInLastUsedWorld.Emplace(Volume);
	}
}

void SVisualLogger::ProcessFilterVolumes()
{
	FilterBoxes.Reset(FilterVolumesInLastUsedWorld.Num());
	for (TWeakObjectPtr<const AVisualLoggerFilterVolume>& WeakVolume : FilterVolumesInLastUsedWorld)
	{
		if (const AVisualLoggerFilterVolume* Volume = WeakVolume.Get())
		{
			FilterBoxes.Add(Volume->GetBounds().GetBox());
		}
	}
}

void SVisualLogger::OnLevelActorAdded(AActor* Actor)
{
	if (AVisualLoggerFilterVolume* Volume = Cast<AVisualLoggerFilterVolume>(Actor))
	{
		FilterVolumesInLastUsedWorld.AddUnique(Volume);
		ProcessFilterVolumes();
		OnFiltersChanged();
	}
}

void SVisualLogger::OnLevelActorDeleted(AActor* Actor)
{
	if (AVisualLoggerFilterVolume* Volume = Cast<AVisualLoggerFilterVolume>(Actor))
	{
		FilterVolumesInLastUsedWorld.Remove(Volume);
		ProcessFilterVolumes();
		OnFiltersChanged();
	}
}

void SVisualLogger::OnActorMoved(AActor* Actor)
{
	if (AVisualLoggerFilterVolume* Volume = Cast<AVisualLoggerFilterVolume>(Actor))
	{
		ProcessFilterVolumes();
		OnFiltersChanged();
	}
}

void SVisualLogger::OnSettingsChanged(FName PropertyName)
{
	if (PropertyName == GET_MEMBER_NAME_CHECKED(ULogVisualizerSettings, bUseFilterVolumes))
	{
		// Note that we don't need to collect & process the volumes since their bookkeeping
		// is not conditional to the flag, only the filtering.
		OnFiltersChanged();
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(ULogVisualizerSettings, bSearchInsideLogs))
	{
		OnFiltersChanged();
	}
}

void SVisualLogger::OnNewWorld(UWorld* NewWorld)
{
	LastUsedWorld = NewWorld;

	// Only reset data when activating a new game world, not when returning to the Editor world (i.e. after PIE)
	if (!IsValid(NewWorld) ||
		(NewWorld->IsGameWorld() && GetDefault<ULogVisualizerSettings>()->bResetDataWithNewSession))
	{
		ResetData();
	}

	CollectFilterVolumes();
	ProcessFilterVolumes();

	AVisualLoggerRenderingActor* HelperActor = Cast<AVisualLoggerRenderingActor>(FVisualLoggerEditorInterface::Get()->GetHelperActor(LastUsedWorld.Get()));
	if (ensure(HelperActor != nullptr))
	{
		// reset data and simulate row/item selection to recreate rendering proxy with correct data
		HelperActor->ResetRendering();
		const TArray<FName>& SelectedRows = FVisualLoggerDatabase::Get().GetSelectedRows();
		HelperActor->ObjectSelectionChanged(SelectedRows);
		for (auto& RowName : SelectedRows)
		{
			FVisualLoggerDBRow& DBRow = FVisualLoggerDatabase::Get().GetRowByName(RowName);
			HelperActor->OnItemSelectionChanged(DBRow, DBRow.GetCurrentItemIndex());
		}
	}
}

void SVisualLogger::OnObjectSelectionChanged(const TArray<FName>& RowNames)
{
	const double ScrubTime = FLogVisualizer::Get().GetTimeSliderController().Get()->GetTimeSliderArgs().ScrubPosition.Get();
	for (auto RowName : RowNames)
	{
		FVisualLoggerDBRow& DBRow = FVisualLoggerDatabase::Get().GetRowByName(RowName);
		if (DBRow.GetCurrentItemIndex() == INDEX_NONE)
		{
			DBRow.MoveTo(DBRow.GetClosestItem(ScrubTime, ScrubTime));
		}
	}
}

void SVisualLogger::OnItemsSelectionChanged(const FVisualLoggerDBRow& ChangedRow, int32 SelectedItemIndex)
{
	const TMap<FName, FVisualLogExtensionInterface*>& AllExtensions = FVisualLogger::Get().GetAllExtensions();
	for (auto Iterator = AllExtensions.CreateConstIterator(); Iterator; ++Iterator)
	{
		FVisualLogExtensionInterface* Extension = (*Iterator).Value;
		if (Extension != NULL)
		{
			Extension->OnItemsSelectionChanged(FVisualLoggerEditorInterface::Get());
		}
	}
}

void SVisualLogger::OnFiltersChanged()
{
	const uint32 StartCycles = FPlatformTime::Cycles();

	TArray<TFuture<void> > AllFutures;
	for (auto Iterator = FVisualLoggerDatabase::Get().GetRowIterator(); Iterator; ++Iterator)
	{
		FVisualLoggerDBRow* DBRow = &(*Iterator);
		AllFutures.Add(
			Async(EAsyncExecution::TaskGraph, [this, DBRow]()
			{
				const TArray<FVisualLogDevice::FVisualLogEntryItem>& Entries = DBRow->GetItems();
				for (int32 Index = 0; Index < Entries.Num(); ++Index)
				{
					UpdateVisibilityForEntry(*DBRow, Index);
				}
			}
		));
	}

	bool bAllFuturesReady = false;
	do
	{
		bAllFuturesReady = true;
		for (TFuture<void>& CurrentFuture : AllFutures)
		{
			bAllFuturesReady &= CurrentFuture.IsReady();
			if (bAllFuturesReady == false)
			{
				break;
			}
		}
		if (bAllFuturesReady == false)
		{
			FPlatformProcess::Sleep(0.01f);
		}
	} while (bAllFuturesReady != true);

	for (auto Iterator = FVisualLoggerDatabase::Get().GetRowIterator(); Iterator; ++Iterator)
	{
		FVisualLoggerDBRow& DBRow = *Iterator;
		FVisualLoggerDatabase::Get().SetRowVisibility(DBRow.GetOwnerName(), DBRow.IsRowVisible());
	}

	const int32 BlockingCycles = int32(FPlatformTime::Cycles() - StartCycles);
	{
		const TArray<FName>& SelectedRows = FVisualLoggerDatabase::Get().GetSelectedRows();
		const double ScrubTime = FLogVisualizer::Get().GetTimeSliderController()->GetTimeSliderArgs().ScrubPosition.Get();
		{
			for (auto RowName : SelectedRows)
			{
				auto& DBRow = FVisualLoggerDatabase::Get().GetRowByName(RowName);
				const int32 ClosestItem = DBRow.GetClosestItem(ScrubTime, ScrubTime);
				const TArray<FVisualLogDevice::FVisualLogEntryItem>& Items = DBRow.GetItems();
				if (Items.IsValidIndex(ClosestItem) && Items[ClosestItem].Entry.TimeStamp <= ScrubTime)
				{
					DBRow.MoveTo(ClosestItem);
				}
			}
		}
	}
	UE_LOGF(LogVisualLogger, Display, "SVisualLogger::OnFiltersChanged: %5.2fms", FPlatformTime::ToMilliseconds(BlockingCycles));
}

void SVisualLogger::OnFiltersSearchChanged(const FText& Filter)
{
	const uint32 StartCycles = FPlatformTime::Cycles();

	FVisualLoggerFilters::Get().SetSearchString(Filter.ToString());

	TArray<TFuture<void> > AllFutures;
	for (auto Iterator = FVisualLoggerDatabase::Get().GetRowIterator(); Iterator; ++Iterator)
	{
		FVisualLoggerDBRow* DBRow = &(*Iterator);
			AllFutures.Add(
				Async(EAsyncExecution::TaskGraph, [this, DBRow]()
					{
						const TArray<FVisualLogDevice::FVisualLogEntryItem>& Entries = DBRow->GetItems();
						for (int32 Index = 0; Index < Entries.Num(); ++Index)
						{
							UpdateVisibilityForEntry(*DBRow, Index);
						}
					}
				)
		);
	}

	bool bAllFuturesReady = false;
	do 
	{
		bAllFuturesReady = true;
		for (TFuture<void>& CurrentFuture : AllFutures)
		{
			bAllFuturesReady &= CurrentFuture.IsReady();
			if (bAllFuturesReady == false)
			{
				break;
			}
		}
		if (bAllFuturesReady == false)
		{
			FPlatformProcess::Sleep(0.01f);
		}
	} while (bAllFuturesReady != true);

	for (auto Iterator = FVisualLoggerDatabase::Get().GetRowIterator(); Iterator; ++Iterator)
	{
		FVisualLoggerDBRow& DBRow = *Iterator;
		FVisualLoggerDatabase::Get().SetRowVisibility(DBRow.GetOwnerName(), DBRow.IsRowVisible());
	}

	if (LogsList.IsValid())
	{
		// it depends on rows visibility so it have to be called here, manually after changes to rows visibilities
		LogsList->OnFiltersSearchChanged(Filter);
	}

	if (VisualLoggerCanvasRenderer.IsValid())
	{
		VisualLoggerCanvasRenderer->DirtyCachedData();
	}

	const int32 BlockingCycles = int32(FPlatformTime::Cycles() - StartCycles);
	UE_LOGF(LogVisualLogger, Display, "SVisualLogger::OnFiltersSearchChanged: %5.2fms", FPlatformTime::ToMilliseconds(BlockingCycles));
}

void SVisualLogger::OnNewItemHandler(const FVisualLoggerDBRow& DBRow, int32 ItemIndex)
{
	UpdateVisibilityForEntry(DBRow, ItemIndex);

	// A new item can be hidden by the category filters by UpdateVisibilityForEntry.
	// In such case, we might need to update row visibility/
	// Note that this is not called within UpdateVisibilityForEntry since it can be called by async tasks.
	FVisualLoggerDatabase::Get().SetRowVisibility(DBRow.GetOwnerName(), DBRow.IsRowVisible());

	if (bAutoScrollToLastItem)
	{
		TGuardValue<bool> Guard(bAutoScrollingToLastItem, true);
		FLogVisualizer::Get().GotoLastItemAnyRow();
	}
}

void SVisualLogger::UpdateVisibilityForEntry(const FVisualLoggerDBRow& DBRow, int32 ItemIndex)
{
	const ULogVisualizerSettings* Settings = GetDefault<ULogVisualizerSettings>();
	const FVisualLogDevice::FVisualLogEntryItem& CurrentEntry = DBRow.GetItems()[ItemIndex];
	const FString SearchString = FVisualLoggerFilters::Get().GetSearchString();

	TArray<FVisualLoggerCategoryVerbosityPair> OutCategories;
	FVisualLoggerHelpers::GetCategories(CurrentEntry.Entry, OutCategories);

	bool bPassingFilter = false;

	for (FVisualLoggerCategoryVerbosityPair& CategoryPair : OutCategories)
	{
		if (FVisualLoggerFilters::Get().ShouldDisplayCategory(CategoryPair.CategoryName, CategoryPair.Verbosity))
		{
			bPassingFilter = true;
			break;
		}
	}

	if (bPassingFilter && Settings->bUseFilterVolumes && FilterBoxes.Num() > 0 && CurrentEntry.Entry.bIsLocationValid)
	{
		bool bIsInsideAnyFilterVolume = false;
		for (const FBox& Box : FilterBoxes)
		{
			if (Box.IsInside(CurrentEntry.Entry.Location))
			{
				bIsInsideAnyFilterVolume = true;
				break;
			}
		}

		bPassingFilter &= bIsInsideAnyFilterVolume;
	}

	if (bPassingFilter && Settings->bSearchInsideLogs && SearchString.Len() > 0)
	{
		bPassingFilter = false;
		for (const FVisualLogLine& CurrentLine : CurrentEntry.Entry.LogLines)
		{
			if (CurrentLine.Line.Find(SearchString) != INDEX_NONE || CurrentLine.Category.ToString().Find(SearchString) != INDEX_NONE)
			{
				bPassingFilter = true;
				break;
			}
		}

		if (!bPassingFilter)
		{
			for (const FVisualLogEvent& CurrentEvent : CurrentEntry.Entry.Events)
			{
				if (CurrentEvent.Name.Find(SearchString) != INDEX_NONE)
				{
					bPassingFilter = true;
					break;
				}
			}
		}
	}

	FVisualLoggerDatabase::Get().GetRowByName(DBRow.GetOwnerName()).SetItemVisibility(ItemIndex, bPassingFilter);
}

void SVisualLogger::OnLogLineSelectionChanged(TSharedPtr<struct FLogEntryItem> SelectedItem, int64 UserData, FName TagName)
{
	const TMap<FName, FVisualLogExtensionInterface*>& AllExtensions = FVisualLogger::Get().GetAllExtensions();
	for (auto Iterator = AllExtensions.CreateConstIterator(); Iterator; ++Iterator)
	{
		FVisualLogExtensionInterface* Extension = (*Iterator).Value;
		if (Extension != NULL)
		{
			Extension->OnLogLineSelectionChanged(FVisualLoggerEditorInterface::Get(), SelectedItem, UserData);
		}
	}
}

void SVisualLogger::OnScrubPositionChanged(double NewScrubPosition, bool bScrubbing)
{
	const TArray<FName> &SelectedRows = FVisualLoggerDatabase::Get().GetSelectedRows();
	const double ScrubTime = FLogVisualizer::Get().GetTimeSliderController()->GetTimeSliderArgs().ScrubPosition.Get();
	for (auto RowName : SelectedRows)
	{
		auto& DBRow = FVisualLoggerDatabase::Get().GetRowByName(RowName);
		const int32 ClosestItem = SelectedRows.Num() > 1 ? DBRow.GetClosestItem(NewScrubPosition, ScrubTime) : DBRow.GetClosestItem(NewScrubPosition);
		const TArray<FVisualLogDevice::FVisualLogEntryItem>& Items = DBRow.GetItems();
		if (Items.IsValidIndex(ClosestItem) && Items[ClosestItem].Entry.TimeStamp <= NewScrubPosition)
		{
			DBRow.MoveTo(ClosestItem);
		}
	}

	const TMap<FName, FVisualLogExtensionInterface*>& AllExtensions = FVisualLogger::Get().GetAllExtensions();
	for (auto Iterator = AllExtensions.CreateConstIterator(); Iterator; ++Iterator)
	{
		FVisualLogExtensionInterface* Extension = (*Iterator).Value;
		if (Extension != NULL)
		{
			Extension->OnScrubPositionChanged(FVisualLoggerEditorInterface::Get(), NewScrubPosition, bScrubbing);
		}
	}

	// Cancel auto-scrolling on scrub (except if it comes from within and the actual bAutoScroll behavior, which is guarded by bAutoScrollingToLastItem)
	if (!bAutoScrollingToLastItem)
	{
		bAutoScrollToLastItem = false;
	}
}

FReply SVisualLogger::OnKeyboardRedirection(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	FReply ReturnValue = FReply::Unhandled();

	const TArray<FName>& SelectedRows = FVisualLoggerDatabase::Get().GetSelectedRows();
	if (SelectedRows.Num() == 0)
	{
		return ReturnValue;
	}

	// find time to move by
	const FKey Key = InKeyEvent.GetKey();
	if (Key == EKeys::Left || Key == EKeys::Right)
	{
		const double ScrubTime = FLogVisualizer::Get().GetTimeSliderController()->GetTimeSliderArgs().ScrubPosition.Get();
		double NewTimeToSet = ScrubTime;
		double BestTimeDifference = MAX_dbl;

		const int32 MoveDist = InKeyEvent.IsLeftControlDown() ? InKeyEvent.IsLeftShiftDown() ? 20 : 10 : 1;
		for (auto RowName : SelectedRows)
		{
			const FVisualLoggerDBRow& DBRow = FVisualLoggerDatabase::Get().GetRowByName(RowName);
			const int32 CurrentItemIndex = DBRow.GetCurrentItemIndex();
			if (CurrentItemIndex == INDEX_NONE)
			{
				continue;
			}

			if (Key == EKeys::Right)
			{
				double TimeDifference = DBRow.GetCurrentItem().Entry.TimeStamp - ScrubTime;
				if (TimeDifference > 0 && FMath::Abs(TimeDifference) < FMath::Abs(BestTimeDifference))
				{
					BestTimeDifference = TimeDifference;
					NewTimeToSet = DBRow.GetCurrentItem().Entry.TimeStamp;
				}

				const int32 NextItemIndex = FLogVisualizer::Get().GetNextItem(RowName, MoveDist);
				TimeDifference = DBRow.GetItems()[NextItemIndex].Entry.TimeStamp - ScrubTime;
				if (TimeDifference > 0 && FMath::Abs(TimeDifference) < FMath::Abs(BestTimeDifference))
				{
					BestTimeDifference = TimeDifference;
					NewTimeToSet = DBRow.GetItems()[NextItemIndex].Entry.TimeStamp;
				}
			}
			else if (Key == EKeys::Left)
			{
				double TimeDifference = DBRow.GetCurrentItem().Entry.TimeStamp - ScrubTime;
				if (TimeDifference < 0 && FMath::Abs(TimeDifference) < FMath::Abs(BestTimeDifference))
				{
					BestTimeDifference = TimeDifference;
					NewTimeToSet = DBRow.GetCurrentItem().Entry.TimeStamp;
				}

				const int32 PrevItemIndex = FLogVisualizer::Get().GetPreviousItem(RowName, MoveDist);
				TimeDifference = DBRow.GetItems()[PrevItemIndex].Entry.TimeStamp - ScrubTime;
				if (TimeDifference < 0 && FMath::Abs(TimeDifference) > 0 && FMath::Abs(TimeDifference) < FMath::Abs(BestTimeDifference))
				{
					BestTimeDifference = TimeDifference;
					NewTimeToSet = DBRow.GetItems()[PrevItemIndex].Entry.TimeStamp;
				}
			}
		}

		FLogVisualizer::Get().GetTimeSliderController()->CommitScrubPosition(NewTimeToSet, false);
		ReturnValue = FReply::Handled();
	}

	FName OwnerName = SelectedRows[SelectedRows.Num() - 1];
	const FVisualLoggerDBRow& DBRow = FVisualLoggerDatabase::Get().GetRowByName(OwnerName);
	if (DBRow.GetCurrentItemIndex() != INDEX_NONE)
	{
		if (Key == EKeys::Home)
		{
			FLogVisualizer::Get().GotoFirstItem(OwnerName);
			ReturnValue = FReply::Handled();
		}
		else if (Key == EKeys::End)
		{
			FLogVisualizer::Get().GotoLastItem(OwnerName);
			ReturnValue = FReply::Handled();
		}
		else if (Key == EKeys::Enter)
		{
			FLogVisualizer::Get().UpdateCameraPosition(OwnerName, DBRow.GetCurrentItemIndex());
			ReturnValue = FReply::Handled();
		}
	}

	return ReturnValue;
}

#undef LOCTEXT_NAMESPACE
#endif // UE_DEBUG_VISUALIZER_TOOL_ENABLED