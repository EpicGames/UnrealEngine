// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "VisualLogger/VisualLoggerDefines.h"

#if UE_DEBUG_VISUALIZER_TOOL_ENABLED
#include "Framework/Commands/UICommandList.h"
#include "Framework/Docking/TabManager.h"
#include "Input/Reply.h"
#include "VisualLogger/VisualLoggerTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class AVisualLoggerCameraController;
class FMenuBuilder;
class SVisualLoggerFilters;
class SVisualLoggerLogsList;
class SVisualLoggerStatusView;
class SVisualLoggerView;
struct FLogEntryItem;
struct FVisualLoggerCanvasRenderer;
struct FVisualLoggerDBRow;

class SVisualLogger : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SVisualLogger) { }
	SLATE_END_ARGS()

public:

	/**
	* Default constructor.
	*/
	SVisualLogger();
	virtual ~SVisualLogger();

public:

	/**
	* Constructs the application.
	*
	* @param InArgs The Slate argument list.
	* @param ConstructUnderMajorTab The major tab which will contain the session front-end.
	* @param ConstructUnderWindow The window in which this widget is being constructed.
	*/
	void Construct(const FArguments& InArgs, const TSharedRef<SDockTab>& ConstructUnderMajorTab, const TSharedPtr<SWindow>& ConstructUnderWindow);

public:
	TSharedRef<SDockTab> HandleTabManagerSpawnTab(const FSpawnTabArgs& Args, FName TabIdentifier) const;
	static void FillWindowMenu(FMenuBuilder& MenuBuilder, const TSharedPtr<FTabManager> TabManager);

	/** Callback for for when the owner tab's visual state is being persisted. */
	void HandleMajorTabPersistVisualState();
	virtual bool SupportsKeyboardFocus() const override { return true; }

	void OnNewLogEntry(const FVisualLogDevice::FVisualLogEntryItem& Entry);
	void OnFiltersChanged();
	void OnObjectSelectionChanged(const TArray<FName>& RowNames);
	void OnItemsSelectionChanged(const FVisualLoggerDBRow&, int32);
	void OnNewItemHandler(const FVisualLoggerDBRow& BDRow, int32 ItemIndex);
	void UpdateVisibilityForEntry(const FVisualLoggerDBRow& BDRow, int32 ItemIndex);
	void OnScrubPositionChanged(double NewScrubPosition, bool bScrubbing);

	void OnFiltersSearchChanged(const FText& Filter);
	void OnLogLineSelectionChanged(TSharedPtr<struct FLogEntryItem> SelectedItem, int64 UserData, FName TagName);
	FReply OnKeyboardRedirection(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent);


	bool HandleStartRecordingCommandCanExecute() const;
	void HandleStartRecordingCommandExecute();
	bool HandleStartRecordingCommandIsVisible() const;

	bool HandleStopRecordingCommandCanExecute() const;
	void HandleStopRecordingCommandExecute();
	bool HandleStopRecordingCommandIsVisible() const;

	bool HandlePauseCommandCanExecute() const;
	void HandlePauseCommandExecute();
	bool HandlePauseCommandIsVisible() const;

	bool HandleResumeCommandCanExecute() const;
	void HandleResumeCommandExecute();
	bool HandleResumeCommandIsVisible() const;

	bool HandleLoadCommandCanExecute() const;
	void HandleLoadCommandExecute();

	void HandleAutoScrollCommandExecute();
	bool HandleAutoScrollIsChecked() const;

	bool HandleSaveCommandCanExecute() const;
	void HandleSaveCommandExecute();

	void HandleSaveAllCommandExecute();

	void HandleSaveCommand(bool bSaveAllData);

	bool HandleRefreshCommandCanExecute() const;
	void HandleRefreshCommandExecute();

	bool HandleCameraCommandCanExecute() const;
	void HandleCameraCommandExecute();
	bool HandleCameraCommandIsChecked() const;

	TSharedPtr<SVisualLoggerFilters> GetVisualLoggerFilters() { return VisualLoggerFilters; }
	void HandleTabManagerPersistLayout(const TSharedRef<FTabManager::FLayout>& LayoutToSave);

	void OnNewWorld(UWorld* NewWorld);
	
	void OnLevelActorAdded(AActor* Actor);
	void OnLevelActorDeleted(AActor* Actor);
	void OnActorMoved(AActor* Actor);

	void OnSettingsChanged(FName PropertyName);

	void ResetData();

protected:
	void CollectFilterVolumes();
	void ProcessFilterVolumes();

	// Holds the list of UI commands.
	TSharedRef<FUICommandList> CommandList;

	// Holds the tab manager that manages the front-end's tabs.
	TSharedPtr<FTabManager> TabManager;

	// Visual Logger device to get and collect logs.
	TSharedPtr<FVisualLogDevice> InternalDevice;

	TWeakObjectPtr<class AVisualLoggerCameraController> CameraController;
	TSharedPtr<struct FVisualLoggerCanvasRenderer> VisualLoggerCanvasRenderer;

	mutable TSharedPtr<SVisualLoggerFilters> VisualLoggerFilters;
	mutable TSharedPtr<SVisualLoggerView> MainView;
	mutable TSharedPtr<SVisualLoggerLogsList> LogsList;
	mutable TSharedPtr<SVisualLoggerStatusView> StatusView;

	bool bPausedLogger;
	TArray<FVisualLogDevice::FVisualLogEntryItem> OnPauseCacheForEntries;

	bool bGotHistogramData;

	// Instructs the visual logger to always scrub to the last item
	bool bAutoScrollToLastItem;
	// Guard value that prevents canceling bAutoScrollToLastItem when we're actually auto-scrubbing (instead of manually scrubbing)
	mutable bool bAutoScrollingToLastItem;

#if WITH_EDITOR
	FDelegateHandle PostWorldInitializationHandle;
#endif

	FDelegateHandle DrawOnCanvasDelegateHandle;
	TWeakObjectPtr<class UWorld> LastUsedWorld;

	TArray<TWeakObjectPtr<const class AVisualLoggerFilterVolume>> FilterVolumesInLastUsedWorld;
	TArray<FBox> FilterBoxes;
};

#endif // UE_DEBUG_VISUALIZER_TOOL_ENABLED