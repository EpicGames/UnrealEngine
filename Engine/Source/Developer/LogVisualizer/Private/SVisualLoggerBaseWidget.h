// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

/* Dependencies
*****************************************************************************/

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"
#include "SVisualLogger.h"
#include "LogVisualizerPublic.h"

class SVisualLoggerBaseWidget : public SCompoundWidget
{
public:
	virtual bool SupportsKeyboardFocus() const override { return true; }
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override
	{
		return FLogVisualizer::Get().GetEvents().OnKeyboardEvent.Execute(MyGeometry, InKeyEvent);
	}
};
