// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "Input/Reply.h"
#include "Widgets/Input/SCheckBox.h"

/** A class for check boxes in the filter list. If you double click a filter check box, you will enable it and disable all others */
class SVisualLoggerFilterCheckBox : public SCheckBox
{
public:
	void SetOnFilterDoubleClicked(const FOnClicked& NewFilterDoubleClicked)
	{
		OnFilterDoubleClicked = NewFilterDoubleClicked;
	}

	void SetOnFilterMiddleButtonClicked(const FOnClicked& NewFilterMiddleButtonClicked)
	{
		OnFilterMiddleButtonClicked = NewFilterMiddleButtonClicked;
	}

	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override
	{
		if (InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton && OnFilterDoubleClicked.IsBound())
		{
			return OnFilterDoubleClicked.Execute();
		}
		else
		{
			return SCheckBox::OnMouseButtonDoubleClick(InMyGeometry, InMouseEvent);
		}
	}

	virtual FReply OnMouseButtonUp(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override
	{
		if (InMouseEvent.GetEffectingButton() == EKeys::MiddleMouseButton && OnFilterMiddleButtonClicked.IsBound())
		{
			return OnFilterMiddleButtonClicked.Execute();
		}
		else
		{
			return SCheckBox::OnMouseButtonUp(InMyGeometry, InMouseEvent);
		}
	}

private:
	FOnClicked OnFilterDoubleClicked;
	FOnClicked OnFilterMiddleButtonClicked;
};
