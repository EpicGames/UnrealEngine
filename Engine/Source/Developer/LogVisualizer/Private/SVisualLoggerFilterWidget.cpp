// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVisualLoggerFilterWidget.h"
#include "Misc/OutputDeviceHelper.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "LogVisualizerSettings.h"
#include "LogVisualizerStyle.h"

#define LOCTEXT_NAMESPACE "SVisualLoggerFilterWidget"
/** Constructs this widget with InArgs */
void SVisualLoggerFilterWidget::Construct(const FArguments& InArgs)
{
	OnFilterChanged = InArgs._OnFilterChanged;
	OnRequestRemove = InArgs._OnRequestRemove;
	OnRequestEnableOnly = InArgs._OnRequestEnableOnly;
	OnRequestDisableAll = InArgs._OnRequestDisableAll;
	OnRequestRemoveAll = InArgs._OnRequestRemoveAll;
	FilterName = InArgs._FilterName;
	ColorCategory = InArgs._ColorCategory;
	BorderBackgroundColor = FLinearColor(0.2f, 0.2f, 0.2f, 0.2f);

	LastVerbosity = ELogVerbosity::NoLogging;
	GetCaptionString(); // cache caption string

	bWasEnabledLastTime = !IsEnabled();
	GetTooltipString(); // cache tooltip string

	// Get the tooltip and color of the type represented by this filter
	FilterColor = ColorCategory;

	ChildSlot
		[
			SNew(SBorder)
			.Padding(2.f)
			.BorderBackgroundColor(this, &SVisualLoggerFilterWidget::GetBorderBackgroundColor)
			.BorderImage(FLogVisualizerStyle::Get().GetBrush("ContentBrowser.FilterButtonBorder"))
			[
				SAssignNew(ToggleButtonPtr, SVisualLoggerFilterCheckBox)
				.Style(FLogVisualizerStyle::Get(), "ContentBrowser.FilterButton")
				.ToolTipText(this, &SVisualLoggerFilterWidget::GetTooltipString)
				.Padding(this, &SVisualLoggerFilterWidget::GetFilterNamePadding)
				.IsChecked(this, &SVisualLoggerFilterWidget::IsChecked)
				.OnCheckStateChanged(this, &SVisualLoggerFilterWidget::FilterToggled)
				.OnGetMenuContent(this, &SVisualLoggerFilterWidget::GetRightClickMenuContent)
				.ForegroundColor(this, &SVisualLoggerFilterWidget::GetFilterForegroundColor)
				[
					SNew(STextBlock)
					.ColorAndOpacity(this, &SVisualLoggerFilterWidget::GetFilterNameColorAndOpacity)
					.Text(this, &SVisualLoggerFilterWidget::GetCaptionString)
					.ShadowColorAndOpacity(FLinearColor::Black)
					.ShadowOffset(FVector2D(1.0f, 1.0f))
				]
			]
		];

	ToggleButtonPtr->SetOnFilterDoubleClicked(FOnClicked::CreateSP(this, &SVisualLoggerFilterWidget::FilterDoubleClicked));
	ToggleButtonPtr->SetOnFilterMiddleButtonClicked(FOnClicked::CreateSP(this, &SVisualLoggerFilterWidget::FilterMiddleButtonClicked));
}

FText SVisualLoggerFilterWidget::GetCaptionString() const
{
	FString CaptionString;
	FCategoryFilter& CategoryFilter = FVisualLoggerFilters::Get().GetCategoryByName(GetFilterNameAsString());
	if (CategoryFilter.LogVerbosity != LastVerbosity)
	{
		const FString VerbosityStr = ::ToString((ELogVerbosity::Type)CategoryFilter.LogVerbosity);
		if (CategoryFilter.LogVerbosity != ELogVerbosity::VeryVerbose)
		{
			CaptionString = FString::Printf(TEXT("%s [%s]"), *GetFilterNameAsString().Replace(TEXT("Log"), TEXT(""), ESearchCase::CaseSensitive), *VerbosityStr.Mid(0, 1));
		}
		else
		{
			CaptionString = FString::Printf(TEXT("%s [VV]"), *GetFilterNameAsString().Replace(TEXT("Log"), TEXT(""), ESearchCase::CaseSensitive));
		}

		CachedCaptionString = FText::FromString(CaptionString);
		LastVerbosity = (ELogVerbosity::Type)CategoryFilter.LogVerbosity;
	}
	return CachedCaptionString;
}

FText SVisualLoggerFilterWidget::GetTooltipString() const
{
	if (bWasEnabledLastTime != IsEnabled())
	{
		FCategoryFilter& CategoryFilter = FVisualLoggerFilters::Get().GetCategoryByName(GetFilterNameAsString());
		const FString VerbosityStr = ::ToString((ELogVerbosity::Type)CategoryFilter.LogVerbosity);

		CachedTooltipString = FText::FromString(
			IsEnabled() ?
			FString::Printf(TEXT("Enabled '%s' category for '%s' verbosity and lower\nRight click to change verbosity"), *GetFilterNameAsString(), *VerbosityStr)
			:
			FString::Printf(TEXT("Disabled '%s' category"), *GetFilterNameAsString())
			);

		bWasEnabledLastTime = IsEnabled();
	}

	return CachedTooltipString;
}

TSharedRef<SWidget> SVisualLoggerFilterWidget::GetRightClickMenuContent()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	if (IsEnabled())
	{
		FText FiletNameAsText = FText::FromString(GetFilterNameAsString());
		MenuBuilder.BeginSection("VerbositySelection", LOCTEXT("VerbositySelection", "Current verbosity selection"));
		{
			for (int32 Index = ELogVerbosity::NoLogging + 1; Index <= ELogVerbosity::VeryVerbose; Index++)
			{
				const FString VerbosityStr = ::ToString((ELogVerbosity::Type)Index);
				MenuBuilder.AddMenuEntry(
					FText::Format(LOCTEXT("UseVerbosity", "Use: {0}"), FText::FromString(VerbosityStr)),
					LOCTEXT("UseVerbosityTooltip", "Apply verbosity to selected filter."),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateSP(this, &SVisualLoggerFilterWidget::SetVerbosityFilter, Index))
					);
			}
		}
		MenuBuilder.EndSection();
	}
	MenuBuilder.BeginSection("FilterAction", LOCTEXT("FilterAction", "Context actions"));
		MenuBuilder.AddMenuEntry(
			LOCTEXT("DisableAllButThis", "Disable all but this"),
			LOCTEXT("HideAllButThisTooltip", "Disable all other categories"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SVisualLoggerFilterWidget::DisableAllButThis))
			);
		MenuBuilder.AddMenuEntry(
			LOCTEXT("EnableAll", "Enable all categories"),
			LOCTEXT("EnableAllTooltip", "Enable all categories"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SVisualLoggerFilterWidget::EnableAllCategories))
			);
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

bool SVisualLoggerFilterWidget::IsEnabled() const 
{
	const FCategoryFilter& CategoryFilter = FVisualLoggerFilters::Get().GetCategoryByName(GetFilterNameAsString());
	return CategoryFilter.Enabled;
}

void SVisualLoggerFilterWidget::SetEnabled(const bool InEnabled)
{
	FCategoryFilter& CategoryFilter = FVisualLoggerFilters::Get().GetCategoryByName(GetFilterNameAsString());
	if (InEnabled != CategoryFilter.Enabled)
	{
		CategoryFilter.Enabled = InEnabled;
		OnFilterChanged.ExecuteIfBound();
	}
}

void SVisualLoggerFilterWidget::FilterToggled(const ECheckBoxState NewState)
{
	SetEnabled(NewState == ECheckBoxState::Checked);
}

void SVisualLoggerFilterWidget::SetVerbosityFilter(int32 SelectedVerbosityIndex)
{
	FCategoryFilter& CategoryFilter = FVisualLoggerFilters::Get().GetCategoryByName(GetFilterNameAsString());
	CategoryFilter.LogVerbosity = (ELogVerbosity::Type)SelectedVerbosityIndex;
	OnFilterChanged.ExecuteIfBound();
}

void SVisualLoggerFilterWidget::DisableAllButThis()
{
	FVisualLoggerFilters::Get().DeactivateAllButThis(GetFilterNameAsString());
	OnFilterChanged.ExecuteIfBound();
}

void SVisualLoggerFilterWidget::EnableAllCategories()
{
	FVisualLoggerFilters::Get().EnableAllCategories();
	OnFilterChanged.ExecuteIfBound();
}


#undef LOCTEXT_NAMESPACE
