// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateColor.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Layout/Margin.h"
#include "Styling/SlateTypes.h"
#include "Widgets/SCompoundWidget.h"
#include "SVisualLoggerFilterCheckBox.h"

/**
* A single filter in the filter list. Can be removed by clicking the remove button on it.
*/
class SVisualLoggerFilterWidget : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnFilterRequest, const TSharedRef<SVisualLoggerFilterWidget>&);
	DECLARE_DELEGATE(FOnSimpleRequest);

	SLATE_BEGIN_ARGS(SVisualLoggerFilterWidget){}
		/** If this is an front end filter, this is the filter object */
	SLATE_ARGUMENT(FName, FilterName)
		/** Color selected for filter*/
		SLATE_ARGUMENT(FLinearColor, ColorCategory)
		/** Invoked when the filter toggled */
		SLATE_EVENT(FOnSimpleRequest, OnFilterChanged)
		/** Invoked when a request to remove this filter originated from within this filter */
		SLATE_EVENT(FOnFilterRequest, OnRequestRemove)
		/** Invoked when a request to enable only this filter originated from within this filter */
		SLATE_EVENT(FOnFilterRequest, OnRequestEnableOnly)
		/** Invoked when a request to disable all filters originated from within this filter */
		SLATE_EVENT(FOnSimpleRequest, OnRequestDisableAll)
		/** Invoked when a request to remove all filters originated from within this filter */
		SLATE_EVENT(FOnSimpleRequest, OnRequestRemoveAll)
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs);

	/** Caption String getter */
	FText GetCaptionString() const;
	/** Tool Tip string getter */
	FText GetTooltipString() const;

	/** Sets whether or not this filter is applied to the combined filter */
	void SetEnabled(bool InEnabled);

	/** Returns true if this filter contributes to the combined filter */
	bool IsEnabled() const;

	FORCEINLINE FName GetFilterName() const { return FilterName; }

	/** Returns the display name for this filter */
	FORCEINLINE FString GetFilterNameAsString() const
	{
		static const FString UnknownFilter(TEXT("UnknownFilter"));
		if (FilterName != NAME_None)
		{
			return FilterName.ToString();
		}

		return UnknownFilter;
	}

	void SetBorderBackgroundColor(FLinearColor InColor) { BorderBackgroundColor = InColor; }

private:
	/** Handler for when the filter checkbox is clicked */
	void FilterToggled(ECheckBoxState NewState);

	/** Handler for when the filter checkbox is double clicked */
	FReply FilterDoubleClicked()
	{
		// Disable all other filters and enable this one.
		OnRequestDisableAll.ExecuteIfBound();
		SetEnabled(true);

		return FReply::Handled();
	}

	/** Handler for when the filter checkbox is middle button clicked */
	FReply FilterMiddleButtonClicked()
	{
		RemoveFilter();
		return FReply::Handled();
	}

	/** Handler to create a right click menu */
	TSharedRef<SWidget> GetRightClickMenuContent();

	/** Removes this filter from the filter list */
	void SetVerbosityFilter(int32 SelectedVerbosityIndex);

	void DisableAllButThis();

	void EnableAllCategories();

	void RemoveFilter()
	{
		TSharedRef<SVisualLoggerFilterWidget> Self = SharedThis(this);
		OnRequestRemove.ExecuteIfBound(Self);
	}

	/** Enables only this filter from the filter list */
	void EnableOnly()
	{
		TSharedRef<SVisualLoggerFilterWidget> Self = SharedThis(this);
		OnRequestEnableOnly.ExecuteIfBound(Self);
	}

	/** Disables all active filters in the list */
	void DisableAllFilters() { OnRequestDisableAll.ExecuteIfBound(); }

	/** Removes all filters in the list */
	void RemoveAllFilters() { OnRequestRemoveAll.ExecuteIfBound(); }

	/** Handler to determine the "checked" state of the filter checkbox */
	ECheckBoxState IsChecked() const { return IsEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; }

	/** Handler to determine the color of the checkbox when it is checked */
	FSlateColor GetFilterForegroundColor() const { return IsChecked() == ECheckBoxState::Checked ? FilterColor : FLinearColor::White; }

	/** Handler to determine the padding of the checkbox text when it is pressed */
	FMargin GetFilterNamePadding() const { return ToggleButtonPtr->IsPressed() ? FMargin(3, 2, 4, 0) : FMargin(3, 1, 4, 1); }

	/** Handler to determine the color of the checkbox text when it is hovered */
	FSlateColor GetFilterNameColorAndOpacity() const
	{
		const float DimFactor = 0.875f;
		constexpr FLinearColor UncheckedColor(0.35f, 0.35f, 0.35f);
		return IsChecked() == ECheckBoxState::Checked ? (IsHovered() ? ColorCategory * DimFactor : ColorCategory) : (IsHovered() ? UncheckedColor : UncheckedColor * DimFactor);
	}

	FSlateColor GetBorderBackgroundColor() const
	{
		return BorderBackgroundColor;
	}

private:
	/** Invoked when the filter toggled */
	FOnSimpleRequest OnFilterChanged;

	/** Invoked when a request to remove this filter originated from within this filter */
	FOnFilterRequest OnRequestRemove;

	/** Invoked when a request to enable only this filter originated from within this filter */
	FOnFilterRequest OnRequestEnableOnly;

	/** Invoked when a request to disable all filters originated from within this filter */
	FOnSimpleRequest OnRequestDisableAll;

	/** Invoked when a request to remove all filters originated from within this filter */
	FOnSimpleRequest OnRequestRemoveAll;

	FName FilterName;

	FLinearColor ColorCategory;

	/** The button to toggle the filter on or off */
	TSharedPtr<SVisualLoggerFilterCheckBox> ToggleButtonPtr;

	/** The color of the checkbox for this filter */
	FLinearColor FilterColor;

	/** Default color for border background */
	FLinearColor BorderBackgroundColor;

	/** hacky to cache data */
	mutable bool bWasEnabledLastTime;
	mutable ELogVerbosity::Type LastVerbosity;

	mutable FText CachedCaptionString;
	mutable FText CachedTooltipString;
};
