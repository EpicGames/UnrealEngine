// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVisualLoggerFilters.h"

#if UE_DEBUG_VISUALIZER_TOOL_ENABLED
#include "EngineGlobals.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SComboButton.h"
#include "LogVisualizerSettings.h"
#include "VisualLoggerDatabase.h"
#include "LogVisualizerStyle.h"
#include "SVisualLoggerFilterWidget.h"
#include "Widgets/Input/SSearchBox.h"
#include "LogVisualizerPublic.h"

#if WITH_EDITOR
#include "Editor/EditorEngine.h"
#include "EditorViewportClient.h"
#endif

/////////////////////
// SLogFilterList
/////////////////////

#define LOCTEXT_NAMESPACE "SVisualLoggerFilters"
void SVisualLoggerFilters::Construct(const FArguments& InArgs, const TSharedRef<FUICommandList>& InCommandList)
{
	ChildSlot
		[
			SAssignNew(FilterBox, SWrapBox)
			.UseAllottedSize(true)
		];

	GraphsFilterCombo =
		SNew(SComboButton)
		.ComboButtonStyle(FLogVisualizerStyle::Get(), "Filters.Style")
		.ForegroundColor(FLinearColor::White)
		.ContentPadding(0.f)
		.OnGetMenuContent(this, &SVisualLoggerFilters::MakeGraphsFilterMenu)
		.ToolTipText(LOCTEXT("AddFilterToolTip", "Add an asset filter."))
		.HasDownArrow(true)
		.ContentPadding(FMargin(1, 0))
		.ButtonContent()
		[
			SNew(STextBlock)
			.TextStyle(FLogVisualizerStyle::Get(), "GenericFilters.TextStyle")
			.Text(LOCTEXT("GraphFilters", "Graph Filters"))
		];


	FilterBox->AddSlot()
		.Padding(3, 3)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(FMargin(0))
			.AutoWidth()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Visibility_Lambda([this]()->EVisibility{ return this->GraphsSearchString.Len() > 0 ? EVisibility::Visible : EVisibility::Hidden; })
				.Image(FLogVisualizerStyle::Get().GetBrush("Filters.FilterIcon"))
			]
			+ SHorizontalBox::Slot()
			.Padding(FMargin(0))
			[
				GraphsFilterCombo.ToSharedRef()
			]
		];

	for (auto& CurrentCategory : FVisualLoggerFilters::Get().Categories)
	{
		AddFilterCategory(CurrentCategory.CategoryName, (ELogVerbosity::Type)CurrentCategory.LogVerbosity, false);
	}

	GraphsFilterCombo->SetVisibility(CachedDatasPerGraph.Num() ? EVisibility::Visible : EVisibility::Collapsed);

	FVisualLoggerFilters::Get().OnFilterCategoryAdded.AddRaw(this, &SVisualLoggerFilters::OnFilterCategoryAdded);
	FVisualLoggerFilters::Get().OnFilterCategoryRemoved.AddRaw(this, &SVisualLoggerFilters::OnFilterCategoryRemoved);
	FVisualLoggerDatabase::Get().GetEvents().OnItemSelectionChanged.AddRaw(this, &SVisualLoggerFilters::OnItemsSelectionChanged);

	FVisualLoggerDatabase::Get().GetEvents().OnGraphAddedEvent.AddRaw(this, &SVisualLoggerFilters::OnGraphAddedEvent);
	FVisualLoggerDatabase::Get().GetEvents().OnGraphDataNameAddedEvent.AddRaw(this, &SVisualLoggerFilters::OnGraphDataNameAddedEvent);
}

SVisualLoggerFilters::~SVisualLoggerFilters()
{
	FVisualLoggerFilters::Get().OnFilterCategoryAdded.RemoveAll(this);
	FVisualLoggerFilters::Get().OnFilterCategoryRemoved.RemoveAll(this);
	FVisualLoggerDatabase::Get().GetEvents().OnItemSelectionChanged.RemoveAll(this);

	FVisualLoggerDatabase::Get().GetEvents().OnGraphAddedEvent.RemoveAll(this);
	FVisualLoggerDatabase::Get().GetEvents().OnGraphDataNameAddedEvent.RemoveAll(this);
}

void SVisualLoggerFilters::ResetData()
{
	for (auto& CurrentFilter : Filters)
	{
		FilterBox->RemoveSlot(CurrentFilter);
	}
	Filters.Reset();
	CachedDatasPerGraph.Reset();

	GraphsSearchString.Reset();
	CachedGraphFilters.Reset();
}

bool SVisualLoggerFilters::GraphSubmenuVisibility(const FName MenuName)
{
	if (GraphsSearchString.Len() == 0)
	{
		return true;
	}

	if (CachedDatasPerGraph.Contains(MenuName))
	{
		const TArray<FName>& DataNames = CachedDatasPerGraph[MenuName];
		for (const FName& CurrentData : DataNames)
		{
			if (CurrentData.ToString().Find(GraphsSearchString) != INDEX_NONE)
			{
				return true;
			}
		}
	}

	return false;
}

void SVisualLoggerFilters::OnGraphAddedEvent(const FName& OwnerName, const FName& GraphName)
{
	CachedDatasPerGraph.FindOrAdd(GraphName);
}

void SVisualLoggerFilters::OnGraphDataNameAddedEvent(const FName& OwnerName, const FName& GraphName, const FName& DataName)
{
	CachedDatasPerGraph.FindOrAdd(GraphName).AddUnique(DataName);
}

TSharedRef<SWidget> SVisualLoggerFilters::MakeGraphsFilterMenu()
{
	FMenuBuilder MenuBuilder(true, NULL);

	MenuBuilder.BeginSection(TEXT("Graphs"));
	{
		TSharedRef<SSearchBox> FiltersSearchBox = SNew(SSearchBox)
			.InitialText(FText::FromString(GraphsSearchString))
			.HintText(LOCTEXT("GraphsFilterSearchHint", "Quick find"))
			.OnTextChanged(this, &SVisualLoggerFilters::OnSearchChanged);

		MenuBuilder.AddWidget(FiltersSearchBox, FText::GetEmpty());

		if (CachedDatasPerGraph.Num() > 0)
		{
			for (auto Iter(CachedDatasPerGraph.CreateConstIterator()); Iter; ++Iter)
			{
				if (Iter.Value().Num() == 0)
				{
					continue;
				}
				const FName GraphName = Iter.Key();

				bool bHighlightName = false;
				const TArray<FName> SelectedOwners = FVisualLoggerDatabase::Get().GetSelectedRows();
				for (FName CurrentOwner : SelectedOwners)
				{
					bHighlightName |= FVisualLoggerGraphsDatabase::Get().ContainsGraphByName(CurrentOwner, GraphName);
					if (bHighlightName)
					{
						break;
					}
				}

				const FText& LabelText = bHighlightName ? FText::FromString(FString::Printf(TEXT("* %s"), *GraphName.ToString())) : FText::FromString(FString::Printf(TEXT("  %s"), *GraphName.ToString()));
				MenuBuilder.AddSubMenu(
					LabelText,
					FText::Format(LOCTEXT("FilterByTooltipPrefix", "Filter by {0}"), LabelText),
					FNewMenuDelegate::CreateSP(this, &SVisualLoggerFilters::CreateFiltersMenuCategoryForGraph, GraphName),
					FUIAction(
					FExecuteAction::CreateSP(this, &SVisualLoggerFilters::GraphFilterCategoryClicked, GraphName),
					FCanExecuteAction(),
					FIsActionChecked::CreateSP(this, &SVisualLoggerFilters::IsGraphFilterCategoryInUse, GraphName),
					FIsActionButtonVisible::CreateLambda([this, GraphName]()->bool{return GraphSubmenuVisibility(GraphName); })
					),
					NAME_None,
					EUserInterfaceActionType::ToggleButton
					);
			}
		}
	}
	MenuBuilder.EndSection();


	FDisplayMetrics DisplayMetrics;
	FSlateApplication::Get().GetCachedDisplayMetrics(DisplayMetrics);

	const FVector2D DisplaySize(
		DisplayMetrics.PrimaryDisplayWorkAreaRect.Right - DisplayMetrics.PrimaryDisplayWorkAreaRect.Left,
		DisplayMetrics.PrimaryDisplayWorkAreaRect.Bottom - DisplayMetrics.PrimaryDisplayWorkAreaRect.Top);

	return
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.MaxHeight(DisplaySize.Y * 0.9)
		[
			MenuBuilder.MakeWidget()
		];
}

void SVisualLoggerFilters::GraphFilterCategoryClicked(FName GraphName)
{
	const bool bNewSet = !IsGraphFilterCategoryInUse(GraphName);

	for (auto Iter(CachedDatasPerGraph[GraphName].CreateConstIterator()); Iter; ++Iter)
	{
		const FName& DataName = *Iter;
		FVisualLoggerFilters::Get().DisableGraphData(GraphName, DataName, !bNewSet);
	}

	FLogVisualizer::Get().GetEvents().OnFiltersChanged.Broadcast();
	InvalidateCanvas();
}

bool SVisualLoggerFilters::IsGraphFilterCategoryInUse(FName GraphName) const
{
	bool bInUse = false;

	for (auto Iter(CachedDatasPerGraph[GraphName].CreateConstIterator()); Iter; ++Iter)
	{
		const FName& DataName = *Iter;
		bInUse |= FVisualLoggerFilters::Get().IsGraphDataDisabled(GraphName, DataName) == false;
	}

	return bInUse;
}


void SVisualLoggerFilters::CreateFiltersMenuCategoryForGraph(FMenuBuilder& MenuBuilder, FName GraphName) const
{	
	for (auto Iter(CachedDatasPerGraph[GraphName].CreateConstIterator()); Iter; ++Iter)
	{
		const FName& DataName = *Iter;
		const FText& LabelText = FText::FromString(DataName.ToString());
		MenuBuilder.AddMenuEntry(
			LabelText,
			FText::Format(LOCTEXT("FilterByTooltipPrefix", "Filter by {0}"), LabelText),
			FSlateIcon(),
			FUIAction(
			FExecuteAction::CreateSP(const_cast<SVisualLoggerFilters*>(this), &SVisualLoggerFilters::FilterByTypeClicked, GraphName, DataName),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &SVisualLoggerFilters::IsAssetTypeActionsInUse, GraphName, DataName),
			FIsActionButtonVisible::CreateLambda([this, LabelText]()->bool{return this->GraphsSearchString.Len() == 0 || LabelText.ToString().Find(this->GraphsSearchString) != INDEX_NONE; })),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
			);
	}
}

void SVisualLoggerFilters::FilterByTypeClicked(FName GraphName, FName DataName)
{
	const bool bIsGraphDataDisabled = FVisualLoggerFilters::Get().IsGraphDataDisabled(GraphName, DataName);
	FVisualLoggerFilters::Get().DisableGraphData(GraphName, DataName, !bIsGraphDataDisabled);
	FLogVisualizer::Get().GetEvents().OnFiltersChanged.Broadcast();
	InvalidateCanvas();
}

bool SVisualLoggerFilters::IsAssetTypeActionsInUse(FName GraphName, FName DataName) const
{
	return FVisualLoggerFilters::Get().IsGraphDataDisabled(GraphName, DataName) == false;
}

void SVisualLoggerFilters::OnSearchChanged(const FText& Filter)
{
	GraphsSearchString = Filter.ToString();
	InvalidateCanvas();
}

void SVisualLoggerFilters::InvalidateCanvas()
{
#if WITH_EDITOR
	UEditorEngine *EEngine = Cast<UEditorEngine>(GEngine);
	if (GIsEditor && EEngine != NULL)
	{
		for (FEditorViewportClient* ViewportClient : EEngine->GetAllViewportClients())
		{
			if (ViewportClient)
			{
				ViewportClient->Invalidate();
			}
		}
	}
#endif
}

uint32 SVisualLoggerFilters::GetCategoryIndex(const FString& InFilterName) const
{
	uint32 CategoryIndex = 0;
	for (auto& CurrentFilter : Filters)
	{
		if (CurrentFilter->GetFilterNameAsString() == InFilterName)
		{
			return CategoryIndex;
		}
		CategoryIndex++;
	}

	return INDEX_NONE;
}

void SVisualLoggerFilters::AddFilterCategory(FString InName, ELogVerbosity::Type InVerbosity, bool bMarkAsInUse)
{
	int32 CharIndex = INDEX_NONE;
	if (InName.FindChar('$', CharIndex) == true)
	{
		TArray<FString> GroupAndName;
		InName.ParseIntoArray(GroupAndName, TEXT("$"), true);
		if (ensure(GroupAndName.Num() == 2))
		{
			CachedGraphFilters.FindOrAdd(*GroupAndName[0]).AddUnique(GroupAndName[1]);
			CachedDatasPerGraph.FindOrAdd(*GroupAndName[0]).AddUnique(*GroupAndName[1]);
		}
	}
	else
	{
		for (TSharedRef<SVisualLoggerFilterWidget>& CurrentFilter : Filters)
		{
			if (CurrentFilter->GetFilterNameAsString() == InName)
			{
				return;
			}
		}

		const FLinearColor Color = FLogVisualizer::Get().GetColorForCategory(InName);
		TSharedRef<SVisualLoggerFilterWidget> NewFilter =
			SNew(SVisualLoggerFilterWidget)
			.FilterName(*InName)
			.ColorCategory(Color)
			.OnFilterChanged(SVisualLoggerFilterWidget::FOnSimpleRequest::CreateRaw(this, &SVisualLoggerFilters::OnFiltersChanged));

		Filters.Add(NewFilter);
		FilterBox->AddSlot()
			.Padding(2, 2)
			[
				NewFilter
			];
	}

	if (bMarkAsInUse)
	{
		FVisualLoggerFilters& PresistentFilters = FVisualLoggerFilters::Get();
		for (int32 Index = PresistentFilters.Categories.Num() - 1; Index >= 0; --Index)
		{
			FCategoryFilter& Category = PresistentFilters.Categories[Index];
			if (Category.CategoryName == InName)
			{
				Category.bIsInUse = true;
			}
		}
	}
}

void SVisualLoggerFilters::OnFilterCategoryAdded(FString InName, ELogVerbosity::Type InVerbosity)
{
	AddFilterCategory(InName, InVerbosity, false);
	GraphsFilterCombo->SetVisibility(CachedGraphFilters.Num() > 0 ? EVisibility::Visible : EVisibility::Collapsed);
}

void SVisualLoggerFilters::OnFilterCategoryRemoved(FString InName)
{
	int32 CharIndex = INDEX_NONE;
	if (InName.FindChar('$', CharIndex) == true)
	{
		//FIXME: Removing filter not implemented yet
		ensureMsgf(0, TEXT("Removing filter not implemented yet"));
	}
	else
	{
		for (TSharedRef<SVisualLoggerFilterWidget> CurrentFilter : Filters)
		{
			if (CurrentFilter->GetFilterNameAsString() == InName)
			{
				FilterBox->RemoveSlot(CurrentFilter);
				Filters.Remove(CurrentFilter);
				break;
			}
		}
	}
}

void SVisualLoggerFilters::OnFiltersChanged()
{
	TArray<FString> EnabledFilters;
	for (TSharedRef<SVisualLoggerFilterWidget> CurrentFilter : Filters)
	{
		if (CurrentFilter->IsEnabled())
		{
			EnabledFilters.AddUnique(CurrentFilter->GetFilterName().ToString());
		}
	}

	FLogVisualizer::Get().GetEvents().OnFiltersChanged.Broadcast();
}

void SVisualLoggerFilters::OnFiltersSearchChanged(const FText& Filter)
{

}

void SVisualLoggerFilters::OnItemsSelectionChanged(const FVisualLoggerDBRow& ChangedRow, int32 SelectedItemIndex)
{
	TArray<FVisualLoggerCategoryVerbosityPair> Categories;
	const TArray<FName>& SelectedRows = FVisualLoggerDatabase::Get().GetSelectedRows();
	for (auto& RowName : SelectedRows)
	{
		FVisualLoggerDBRow& DBRow = FVisualLoggerDatabase::Get().GetRowByName(RowName);
		if (DBRow.GetCurrentItemIndex() != INDEX_NONE)
		{
			FVisualLoggerHelpers::GetCategories(DBRow.GetCurrentItem().Entry, Categories);
		}
	}

	for (int32 Index = 0; Index < Filters.Num(); ++Index)
	{
		SVisualLoggerFilterWidget& Filter = Filters[Index].Get();
		Filter.SetBorderBackgroundColor(FLinearColor(0.2f, 0.2f, 0.2f, 0.1f));
		for (const FVisualLoggerCategoryVerbosityPair& Category : Categories)
		{
			if (Filter.GetFilterName() == Category.CategoryName)
			{
				Filter.SetBorderBackgroundColor(FLinearColor(0.2f, 0.2f, 0.2f, 0.45f));
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
#endif // UE_DEBUG_VISUALIZER_TOOL_ENABLED