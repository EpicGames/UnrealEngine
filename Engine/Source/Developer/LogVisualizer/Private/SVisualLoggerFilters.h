// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VisualLogger/VisualLoggerDefines.h"

#if UE_DEBUG_VISUALIZER_TOOL_ENABLED
#include "SVisualLoggerBaseWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Framework/Commands/UICommandList.h"

class FMenuBuilder;
class SComboButton;
class SVisualLoggerFilterWidget;
class SWrapBox;
struct FVisualLoggerDBRow;

/**
* A list of filters currently applied to an asset view.
*/
class SVisualLoggerFilters : public SVisualLoggerBaseWidget
{
public:
	SLATE_BEGIN_ARGS(SVisualLoggerFilters) { }
	SLATE_END_ARGS()
		
	virtual ~SVisualLoggerFilters();
	void Construct(const FArguments& InArgs, const TSharedRef<FUICommandList>& InCommandList);

	uint32 GetCategoryIndex(const FString& InFilterName) const;
	void OnFiltersSearchChanged(const FText& Filter);
	void OnSearchChanged(const FText& Filter);
	bool GraphSubmenuVisibility(const FName MenuName);
	void FilterByTypeClicked(FName InGraphName, FName InDataName);
	bool IsAssetTypeActionsInUse(FName InGraphName, FName InDataName) const;
	void InvalidateCanvas();
	void GraphFilterCategoryClicked(FName MenuCategory);
	bool IsGraphFilterCategoryInUse(FName MenuCategory) const;
	void OnFiltersChanged();
	void ResetData();
	void OnItemSelectionChanged(const struct FVisualLogEntry& EntryItem);

	TSharedRef<SWidget> MakeGraphsFilterMenu();
	void CreateFiltersMenuCategoryForGraph(FMenuBuilder& MenuBuilder, FName MenuCategory) const;
	bool HasFilters() { return Filters.Num() || CachedGraphFilters.Num(); }

protected:
	//void OnNewItemHandler(const FVisualLoggerDBRow& BDRow, int32 ItemIndex);

	void AddFilterCategory(FString, ELogVerbosity::Type, bool bMarkAsInUse);
	void OnFilterCategoryAdded(FString, ELogVerbosity::Type);
	void OnFilterCategoryRemoved(FString);
	void OnItemsSelectionChanged(const FVisualLoggerDBRow& ChangedRow, int32 SelectedItemIndex);

	void OnGraphAddedEvent(const FName& OwnerName, const FName& GraphName);
	void OnGraphDataNameAddedEvent(const FName& OwnerName, const FName& GraphName, const FName& DataName);

protected:
	/** The horizontal box which contains all the filters */
	TSharedPtr<SWrapBox> FilterBox;
	TArray<TSharedRef<SVisualLoggerFilterWidget>> Filters;
	TSharedPtr<SComboButton> GraphsFilterCombo;

	FString GraphsSearchString;
	TMap<FName, TArray<FString> > CachedGraphFilters;

	TMap<FName, TArray<FName> > CachedDatasPerGraph;
};

#endif // UE_DEBUG_VISUALIZER_TOOL_ENABLED