// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVisualLoggerLogsList.h"

#if UE_DEBUG_VISUALIZER_TOOL_ENABLED
#include "SVisualLoggerTableRow.h"
#include "Misc/OutputDeviceHelper.h"
#include "Widgets/Text/STextBlock.h"
#include "LogVisualizerSettings.h"
#include "VisualLoggerDatabase.h"
#include "LogVisualizerStyle.h"
#include "Widgets/Views/SListView.h"
#include "HAL/PlatformApplicationMisc.h"
#include "LogVisualizerPublic.h"

#define LOCTEXT_NAMESPACE "SVisualLoggerLogsList"

namespace ELogsSortMode
{
	enum Type
	{
		ByName,
		ByStartTime,
		ByEndTime,
	};
}

//----------------------------------------------------------------------//
// SVisualLoggerLogsList
//----------------------------------------------------------------------//
void SVisualLoggerLogsList::Construct(const FArguments& InArgs, const TSharedRef<FUICommandList>& InCommandList)
{
	ChildSlot
	[
		SAssignNew(LogsLinesWidget, SListView<TSharedPtr<FLogEntryItem> >)
		.ListItemsSource(&CachedLogEntryLines)
		.SelectionMode(ESelectionMode::Multi)
		.OnSelectionChanged(this, &SVisualLoggerLogsList::LogEntryLineSelectionChanged)
		.OnGenerateRow(this, &SVisualLoggerLogsList::LogEntryLinesGenerateRow)
		.HeaderRow
		(
			SNew(SHeaderRow)
			+ SHeaderRow::Column(SVisualLoggerTableRow::ColumnId_CategoryLabel)
			.FillWidth(0.1f)
			.DefaultLabel(LOCTEXT("VLoggerCategoryLabel", "Category"))

			+ SHeaderRow::Column(SVisualLoggerTableRow::ColumnId_VerbosityLabel)
			.FillWidth(0.1f)
			.DefaultLabel(LOCTEXT("VLoggerVerbosityLabel", "Verbosity"))

			+ SHeaderRow::Column(SVisualLoggerTableRow::ColumnId_LogLabel)
			.FillWidth(0.8f)
			.DefaultLabel(LOCTEXT("VLoggerLogLabel", "Log"))
		)
	];

	FVisualLoggerDatabase::Get().GetEvents().OnItemSelectionChanged.AddRaw(this, &SVisualLoggerLogsList::OnItemSelectionChanged);
	FVisualLoggerDatabase::Get().GetEvents().OnRowSelectionChanged.AddRaw(this, &SVisualLoggerLogsList::ObjectSelectionChanged);
	FLogVisualizer::Get().GetEvents().OnFiltersChanged.AddRaw(this, &SVisualLoggerLogsList::OnFiltersChanged);
}

SVisualLoggerLogsList::~SVisualLoggerLogsList()
{
	FVisualLoggerDatabase::Get().GetEvents().OnItemSelectionChanged.RemoveAll(this);
	FVisualLoggerDatabase::Get().GetEvents().OnRowSelectionChanged.RemoveAll(this);
	FLogVisualizer::Get().GetEvents().OnFiltersChanged.RemoveAll(this);
}

FReply SVisualLoggerLogsList::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::C && (InKeyEvent.IsLeftCommandDown() || InKeyEvent.IsLeftControlDown()))
	{
		FString ClipboardString;
		for (const TSharedPtr<struct FLogEntryItem>& CurrentItem : LogsLinesWidget->GetSelectedItems())
		{
			if (CurrentItem->Category.Len() > 0)
				ClipboardString += CurrentItem->Category + FString(TEXT(" (")) + FString(::ToString(CurrentItem->Verbosity)) + FString(TEXT(") ")) + CurrentItem->Line;
			else
				ClipboardString += CurrentItem->Line;

			ClipboardString += TEXT("\n");
		}
		FPlatformApplicationMisc::ClipboardCopy(*ClipboardString);
		return FReply::Handled();
	}

	return SVisualLoggerBaseWidget::OnKeyDown(MyGeometry, InKeyEvent);
}

void SVisualLoggerLogsList::OnFiltersChanged()
{
	RegenerateLogEntries();
	LogsLinesWidget->RequestListRefresh();
}

TSharedRef<ITableRow> SVisualLoggerLogsList::LogEntryLinesGenerateRow(TSharedPtr<FLogEntryItem> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SVisualLoggerTableRow, OwnerTable, Item)
		.FilterText_Lambda([]
			{
				return GetDefault<ULogVisualizerSettings>()->bSearchInsideLogs ? FText::FromString(FVisualLoggerFilters::Get().GetSearchString()) : FText::GetEmpty();
			});
}

void SVisualLoggerLogsList::OnFiltersSearchChanged(const FText& Filter)
{
	OnFiltersChanged();
}

void SVisualLoggerLogsList::LogEntryLineSelectionChanged(TSharedPtr<FLogEntryItem> SelectedItem, ESelectInfo::Type SelectInfo)
{
	if (SelectedItem.IsValid() == true)
	{
		FLogVisualizer::Get().GetEvents().OnLogLineSelectionChanged.ExecuteIfBound(SelectedItem, SelectedItem->UserData, SelectedItem->TagName);
	}
	else
	{
		FLogVisualizer::Get().GetEvents().OnLogLineSelectionChanged.ExecuteIfBound(SelectedItem, 0, NAME_None);
	}
}

void SVisualLoggerLogsList::ObjectSelectionChanged(const TArray<FName>&)
{
	RegenerateLogEntries();
	LogsLinesWidget->RequestListRefresh();
}

void SVisualLoggerLogsList::ResetData()
{
	CachedLogEntryLines.Reset();
	LogsLinesWidget->RequestListRefresh();
}

void SVisualLoggerLogsList::OnItemSelectionChanged(const FVisualLoggerDBRow& BDRow, int32 ItemIndex)
{
	RegenerateLogEntries();
}

void SVisualLoggerLogsList::RegenerateLogEntries()
{
	CachedLogEntryLines.Reset();

	const TArray<FName> SelectedRows = FVisualLoggerDatabase::Get().GetSelectedRows();
	for (const FName CurrentRow : SelectedRows)
	{
		if (FVisualLoggerDatabase::Get().IsRowVisible(CurrentRow) == false)
		{
			continue;
		}

		const FVisualLoggerDBRow& DBRow = FVisualLoggerDatabase::Get().GetRowByName(CurrentRow);
		const TArray<FVisualLogDevice::FVisualLogEntryItem>& Entries = FVisualLoggerDatabase::Get().GetRowByName(CurrentRow).GetItems();
		
		int32 BestItemIndex = INDEX_NONE;
		double BestDistance = MAX_dbl;
		for (int32 Index = 0; Index < Entries.Num(); Index++)
		{
			auto& CurrentEntryItem = Entries[Index];
			if (DBRow.IsItemVisible(Index) == false)
			{
				continue;
			}

			TArray<FVisualLoggerCategoryVerbosityPair> OutCategories;
			const double CurrentDist = DBRow.GetCurrentItemIndex() == INDEX_NONE ? 0. : FMath::Abs(CurrentEntryItem.Entry.TimeStamp - DBRow.GetCurrentItem().Entry.TimeStamp);
			if (CurrentDist < BestDistance)
			{
				BestDistance = CurrentDist;
				BestItemIndex = Index;
			}
		}

		if (Entries.IsValidIndex(BestItemIndex))
		{
			GenerateLogs(Entries[BestItemIndex], SelectedRows.Num() > 1);
		}
	}
}

void SVisualLoggerLogsList::GenerateLogs(const FVisualLogDevice::FVisualLogEntryItem& LogEntry, bool bGenerateHeader)
{
	TArray<FVisualLoggerCategoryVerbosityPair> OutCategories;
	FVisualLoggerHelpers::GetCategories(LogEntry.Entry, OutCategories);
	bool bHasDisplayedCategory = false;
	for (const FVisualLoggerCategoryVerbosityPair& CurrentCategory : OutCategories)
	{
		if (FVisualLoggerFilters::Get().ShouldDisplayCategory(CurrentCategory.CategoryName, CurrentCategory.Verbosity))
		{
			bHasDisplayedCategory = true;
			break;
		}
	}
	
	if (!bHasDisplayedCategory)
	{
		return;
	}

	if (bGenerateHeader)
	{
		FLogEntryItem EntryItem;
		EntryItem.Category = FString();
		EntryItem.CategoryColor = FLinearColor::Black;
		EntryItem.Verbosity = ELogVerbosity::VeryVerbose;
		EntryItem.Line = LogEntry.OwnerName.ToString();
		EntryItem.UserData = 0;
		EntryItem.TagName = NAME_None;
		CachedLogEntryLines.Add(MakeShareable(new FLogEntryItem(EntryItem)));
	}

	const FVisualLogLine* LogLine = LogEntry.Entry.LogLines.GetData();
	const bool bSearchInsideLogs = GetDefault<ULogVisualizerSettings>()->bSearchInsideLogs;
	for (int LineIndex = 0; LineIndex < LogEntry.Entry.LogLines.Num(); ++LineIndex, ++LogLine)
	{
		if (FVisualLoggerFilters::Get().ShouldDisplayLine(*LogLine, bSearchInsideLogs))
		{
			FLogEntryItem EntryItem;
			EntryItem.Category = LogLine->Category.ToString();
			EntryItem.CategoryColor = FLogVisualizer::Get().GetColorForCategory(LogLine->Category.ToString());
			EntryItem.Verbosity = LogLine->Verbosity;
			EntryItem.Line = LogLine->Line;
			EntryItem.UserData = LogLine->UserData;
			EntryItem.TagName = LogLine->TagName;

			CachedLogEntryLines.Add(MakeShareable(new FLogEntryItem(EntryItem)));
		}
	}

	for (auto& Event : LogEntry.Entry.Events)
	{
		if (FVisualLoggerFilters::Get().ShouldDisplayCategoryByString(Event.Name, Event.Verbosity))
		{
			FLogEntryItem EntryItem;
			EntryItem.Category = Event.Name;
			EntryItem.CategoryColor = FLogVisualizer::Get().GetColorForCategory(*EntryItem.Category);
			EntryItem.Verbosity = Event.Verbosity;
			EntryItem.Line = FString::Printf(TEXT("Registered event: '%s' (%d times)%s"), *Event.Name, Event.Counter, Event.EventTags.Num() ? TEXT("\n") : TEXT(""));
			for (auto& EventTag : Event.EventTags)
			{
				EntryItem.Line += FString::Printf(TEXT("%d times for tag: '%s'\n"), EventTag.Value, *EventTag.Key.ToString());
			}
			EntryItem.UserData = Event.UserData;
			EntryItem.TagName = Event.TagName;

			CachedLogEntryLines.Add(MakeShareable(new FLogEntryItem(EntryItem)));
		}

	}

	LogsLinesWidget->RequestListRefresh();
}
#undef LOCTEXT_NAMESPACE
#endif // UE_DEBUG_VISUALIZER_TOOL_ENABLED