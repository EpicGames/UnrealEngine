// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VisualLogger/VisualLoggerDefines.h"

#if UE_DEBUG_VISUALIZER_TOOL_ENABLED
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Framework/Commands/UICommandList.h"
#include "VisualLogger/VisualLoggerTypes.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "SVisualLoggerBaseWidget.h"

struct FLogEntryItem;
struct FVisualLoggerDBRow;

class SVisualLoggerLogsList : public SVisualLoggerBaseWidget
{
public:
	SLATE_BEGIN_ARGS(SVisualLoggerLogsList){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FUICommandList>& InCommandList);
	virtual ~SVisualLoggerLogsList();

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	void ResetData();
	void ObjectSelectionChanged(const TArray<FName>& Selection);
	void OnItemSelectionChanged(const FVisualLoggerDBRow& BDRow, int32 ItemIndex);
	void OnFiltersChanged();
	void OnFiltersSearchChanged(const FText& Filter);

protected:
	void RegenerateLogEntries();
	void GenerateLogs(const FVisualLogDevice::FVisualLogEntryItem& EntryItem, bool bGenerateHeader);
	void LogEntryLineSelectionChanged(TSharedPtr<FLogEntryItem> SelectedItem, ESelectInfo::Type SelectInfo);
	TSharedRef<ITableRow> LogEntryLinesGenerateRow(TSharedPtr<struct FLogEntryItem> Item, const TSharedRef<STableViewBase>& OwnerTable);

protected:
	TSharedPtr<SListView<TSharedPtr<struct FLogEntryItem> > > LogsLinesWidget;
	TArray<TSharedPtr<struct FLogEntryItem> > CachedLogEntryLines;
};
#endif // UE_DEBUG_VISUALIZER_TOOL_ENABLED