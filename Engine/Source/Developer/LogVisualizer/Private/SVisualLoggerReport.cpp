// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVisualLoggerReport.h"

#if UE_DEBUG_VISUALIZER_TOOL_ENABLED
#include "SlateOptMacros.h"
#include "LogVisualizerStyle.h"
#include "Widgets/Input/SSearchBox.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "SVisualLoggerReport"

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SVisualLoggerReport::Construct(const FArguments& InArgs, TArray< TSharedPtr<class SLogVisualizerTimeline> >& InSelectedItems, TSharedPtr<class SVisualLoggerView> VisualLoggerView)
{
	SelectedItems = InSelectedItems;
	GenerateReportText();

	TArray< TSharedRef< ITextDecorator > > CustomDecorators;
	for (auto& CurrentEvent : CollectedEvents)
	{
		CustomDecorators.Add(SRichTextBlock::HyperlinkDecorator(CurrentEvent, FSlateHyperlinkRun::FOnClick::CreateLambda(
			[this, VisualLoggerView](const FSlateHyperlinkRun::FMetadata& Metadata){ VisualLoggerView->SetSearchString(FText::FromString(Metadata[TEXT("id")])); }
		)));
	}

	this->ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FLogVisualizerStyle::Get().GetBrush("RichText.Background")).HAlign(EHorizontalAlignment::HAlign_Fill)
		[
			SNew(SScrollBox)
			+ SScrollBox::Slot()
			.HAlign(EHorizontalAlignment::HAlign_Fill)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot().AutoHeight().HAlign(HAlign_Right).VAlign(VAlign_Center).Padding(15, 15)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot().MaxWidth(300)
					[
						SNew(SSearchBox)
						.OnTextChanged( FOnTextChanged::CreateLambda(
						[this](FText NewText){
							InteractiveRichText->SetHighlightText(NewText); 
						}))
					]
				]

				+ SVerticalBox::Slot()
				[
					SNew(SBorder)
					.Padding(5.0f)
					.BorderImage(FAppStyle::Get().GetBrush("BoxShadow"))
					[
						SNew(SBorder).Padding(2.f).HAlign(HAlign_Left)
						.BorderImage(FLogVisualizerStyle::Get().GetBrush("RichText.Background"))
						[
							SAssignNew(InteractiveRichText, SRichTextBlock)
							.Text(ReportText)
							.TextStyle(FLogVisualizerStyle::Get(), "RichText.Text")
							.DecoratorStyleSet(&FLogVisualizerStyle::Get())
							.Justification(ETextJustify::Left)
							.Margin(FMargin(20))
							.Decorators(CustomDecorators)
						]
					]
				]
			]
		]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

SVisualLoggerReport::~SVisualLoggerReport()
{
}

void SVisualLoggerReport::GenerateReportText()
{
	FString OutString;
	TArray<FVisualLogEvent> GlobalEventsStats;
	TMap<FString, TArray<FString> >	 EventToObjectsMap;


	OutString.Append(TEXT("<RichText.HeaderText1>Report Details</>\n"));
	for (TSharedPtr<class SLogVisualizerTimeline> LogItem : SelectedItems)
	{
		TArray<FVisualLogEvent> AllEvents;

		for (const FVisualLogDevice::FVisualLogEntryItem& CurrentEntry : LogItem->GetEntries())
		{
			for (const FVisualLogEvent& Event : CurrentEntry.Entry.Events)
			{
				int32 Index = AllEvents.Find(FVisualLogEvent(Event));
				if (Index != INDEX_NONE)
				{
					for (auto& CurrentEventTag : Event.EventTags)
					{
						if (AllEvents[Index].EventTags.Contains(CurrentEventTag.Key))
						{
							AllEvents[Index].EventTags[CurrentEventTag.Key] += CurrentEventTag.Value;
						}
						else
						{
							AllEvents[Index].EventTags.Add(CurrentEventTag.Key, CurrentEventTag.Value);
						}
					}
					AllEvents[Index].Counter++;
				}
				else
				{
					AllEvents.Add(Event);
				}
			}
		}

		bool bPrintNextLine = false;

		if (AllEvents.Num() > 0)
		{
			OutString.Append(FString::Printf(TEXT("    <RichText.HeaderText2>%s</>"), *LogItem->GetName().ToString()));
		}
		for (auto& CurrentEvent : AllEvents)
		{
			for (auto& CurrentEventTag : CurrentEvent.EventTags)
			{
				OutString.Append(FString::Printf(TEXT(" \n        \u2022  <a id=\"%s\" style=\"RichText.Hyperlink\">%s</>  with <RichText.TextBold>%s</> tag occurred    <RichText.TextBold>%d times</>"), *CurrentEvent.Name, *CurrentEvent.Name, *CurrentEventTag.Key.ToString(), CurrentEventTag.Value));
			}
			OutString.Append(FString::Printf(TEXT("\n        \u2022  <a id=\"%s\" style=\"RichText.Hyperlink\">%s</> occurred <RichText.TextBold>%d times</>"), *CurrentEvent.Name, *CurrentEvent.Name, CurrentEvent.Counter));

			bool bJustAdded = false;
			int32 Index = GlobalEventsStats.Find(FVisualLogEvent(CurrentEvent));
			if (Index != INDEX_NONE)
			{
				GlobalEventsStats[Index].Counter += CurrentEvent.Counter;
				EventToObjectsMap[CurrentEvent.Name].AddUnique(LogItem->GetName().ToString());
			}
			else
			{
				bJustAdded = true;
				GlobalEventsStats.Add(CurrentEvent);
				EventToObjectsMap.FindOrAdd(CurrentEvent.Name).Add(LogItem->GetName().ToString());
			}

			Index = GlobalEventsStats.Find(FVisualLogEvent(CurrentEvent));
			for (auto& CurrentEventTag : CurrentEvent.EventTags)
			{
				if (!bJustAdded)
				{
					GlobalEventsStats[Index].EventTags.FindOrAdd(CurrentEventTag.Key) += CurrentEventTag.Value;
				}

				if (EventToObjectsMap.Contains(CurrentEvent.Name + CurrentEventTag.Key.ToString()))
				{
					EventToObjectsMap[CurrentEvent.Name + CurrentEventTag.Key.ToString()].AddUnique(LogItem->GetName().ToString());
				}
				else
				{
					EventToObjectsMap.FindOrAdd(CurrentEvent.Name + CurrentEventTag.Key.ToString()).AddUnique(LogItem->GetName().ToString());
				}
			}
			OutString.Append(TEXT("\n"));
		}
		OutString.Append(TEXT("\n"));
	}

	OutString.Append(TEXT("\n\n<RichText.HeaderText1>Report Summary</>\n"));

	CollectedEvents.Reset();
	for (auto& CurrentEvent : GlobalEventsStats)
	{
		CollectedEvents.Add(*CurrentEvent.Name);
		OutString.Append(FString::Printf(TEXT("    <a id=\"%s\" style=\"RichText.Hyperlink\">%s</>  occurred <RichText.TextBold>%d times</> by %d owners (%s)\n"), *CurrentEvent.Name, *CurrentEvent.Name, CurrentEvent.Counter, EventToObjectsMap.Contains(CurrentEvent.Name) ? EventToObjectsMap[CurrentEvent.Name].Num() : 0, *CurrentEvent.UserFriendlyDesc));
		for (auto& CurrentEventTag : CurrentEvent.EventTags)
		{
			const int32 ObjectsNumber = EventToObjectsMap.Contains(CurrentEvent.Name + CurrentEventTag.Key.ToString()) ? EventToObjectsMap[CurrentEvent.Name + CurrentEventTag.Key.ToString()].Num() : 0;
			OutString.Append(FString::Printf(TEXT("        \u2022  %s to <RichText.TextBold>%s</> tag occurred <RichText.TextBold>%d times</> by %d owners (average %.2f times each)\n"), *CurrentEvent.Name, *CurrentEventTag.Key.ToString(), CurrentEventTag.Value, ObjectsNumber, ObjectsNumber > 0 ? float(CurrentEventTag.Value) / ObjectsNumber : -1));
		}
		OutString.Append(TEXT("\n"));
	}

	ReportText = FText::FromString(OutString);
}


#undef LOCTEXT_NAMESPACE
#endif // UE_DEBUG_VISUALIZER_TOOL_ENABLED