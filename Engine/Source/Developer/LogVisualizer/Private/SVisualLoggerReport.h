// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VisualLogger/VisualLoggerDefines.h"

#if UE_DEBUG_VISUALIZER_TOOL_ENABLED
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "SVisualLoggerView.h"
#include "SVisualLoggerTimeline.h"
#include "Framework/Text/ITextDecorator.h"
#include "Widgets/Text/SRichTextBlock.h"

class SVisualLoggerReport : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SVisualLoggerReport) {}

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TArray< TSharedPtr<class SLogVisualizerTimeline> >&, TSharedPtr<class SVisualLoggerView> VisualLoggerView);

	virtual ~SVisualLoggerReport();

protected:
	void GenerateReportText();

protected:
	TArray< TSharedPtr<class SLogVisualizerTimeline> > SelectedItems;
	TArray<TSharedRef<ITextDecorator>> Decorators;
	TSharedPtr<class SRichTextBlock> InteractiveRichText;
	FText ReportText;
	TArray<FString> CollectedEvents;
};
#endif // UE_DEBUG_VISUALIZER_TOOL_ENABLED
