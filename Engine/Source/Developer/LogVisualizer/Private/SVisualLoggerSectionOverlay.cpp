// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVisualLoggerSectionOverlay.h"
#include "VisualLoggerTimeSliderController.h"


int32 SVisualLoggerSectionOverlay::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	TimeSliderController->OnPaintSectionView( AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, ShouldBeEnabled( bParentEnabled ), bDisplayTickLines.Get(), bDisplayScrubPosition.Get() );

	return SCompoundWidget::OnPaint( Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled  );
}


FReply SVisualLoggerSectionOverlay::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	return TimeSliderController->OnMouseButtonDown(*this, MyGeometry, MouseEvent);
}


FReply SVisualLoggerSectionOverlay::OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	return TimeSliderController->OnMouseButtonUp( *this,  MyGeometry, MouseEvent );
}


FReply SVisualLoggerSectionOverlay::OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	return TimeSliderController->OnMouseMove( *this, MyGeometry, MouseEvent );
}


FReply SVisualLoggerSectionOverlay::OnMouseWheel( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if (MouseEvent.IsLeftShiftDown() || MouseEvent.IsLeftControlDown())
	{
		return TimeSliderController->OnMouseWheel(*this, MyGeometry, MouseEvent);
	}
	return FReply::Unhandled();
}
