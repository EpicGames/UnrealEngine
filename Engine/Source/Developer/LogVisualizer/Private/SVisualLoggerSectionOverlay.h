// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FPaintArgs;
class FSlateWindowElementList;
class FVisualLoggerTimeSliderController;

/**
 * An overlay that displays global information in the section area
 */
class SVisualLoggerSectionOverlay : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SVisualLoggerSectionOverlay )
		: _DisplayTickLines( true )
		, _DisplayScrubPosition( false )
	{}
		SLATE_ATTRIBUTE( bool, DisplayTickLines )
		SLATE_ATTRIBUTE( bool, DisplayScrubPosition )

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<class FVisualLoggerTimeSliderController> InTimeSliderController)
	{
		bDisplayScrubPosition = InArgs._DisplayScrubPosition;
		bDisplayTickLines = InArgs._DisplayTickLines;
		TimeSliderController = InTimeSliderController;
	}

private:
	/** SWidget Interface */
	virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	virtual FReply OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	virtual FReply OnMouseWheel( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;

private:
	/** Controller for manipulating time */
	TSharedPtr<class FVisualLoggerTimeSliderController> TimeSliderController;
	/** Whether or not to display the scrub position */
	TAttribute<bool> bDisplayScrubPosition;
	/** Whether or not to display tick lines */
	TAttribute<bool> bDisplayTickLines;
};
