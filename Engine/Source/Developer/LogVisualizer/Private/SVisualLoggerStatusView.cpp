// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVisualLoggerStatusView.h"
#include "VisualLoggerDatabase.h"
#include "LogVisualizerStyle.h"

#define LOCTEXT_NAMESPACE "SVisualLoggerStatusView"

struct FLogStatusItem
{
	FString ItemText;
	FString ValueText;
	FString HeaderText;

	TArray< TSharedPtr< FLogStatusItem > > Children;

	FLogStatusItem() {}
	FLogStatusItem(const FString& InItemText) : ItemText(InItemText) {}
	FLogStatusItem(const FString& InItemText, const FString& InValueText) : ItemText(InItemText), ValueText(InValueText) {}
};

void SVisualLoggerStatusView::Construct(const FArguments& InArgs, const TSharedRef<FUICommandList>& InCommandList)
{
	ChildSlot
		[
			SNew(SBorder)
			.Padding(1.f)
			.BorderImage(FLogVisualizerStyle::Get().GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				[
					SAssignNew(StatusItemsView, STreeView<TSharedPtr<FLogStatusItem> >)
					.TreeItemsSource(&StatusItems)
					.OnGenerateRow(this, &SVisualLoggerStatusView::HandleGenerateLogStatus)
					.OnGetChildren(this, &SVisualLoggerStatusView::OnLogStatusGetChildren)
					.OnExpansionChanged(this, &SVisualLoggerStatusView::OnExpansionChanged)
					.SelectionMode(ESelectionMode::None)
					.Visibility(EVisibility::Visible)
				]
			]
		];

	FVisualLoggerDatabase::Get().GetEvents().OnRowSelectionChanged.AddRaw(this, &SVisualLoggerStatusView::OnObjectSelectionChanged);
	FVisualLoggerDatabase::Get().GetEvents().OnItemSelectionChanged.AddRaw(this, &SVisualLoggerStatusView::OnItemSelectionChanged);
}

SVisualLoggerStatusView::~SVisualLoggerStatusView()
{
	FVisualLoggerDatabase::Get().GetEvents().OnRowSelectionChanged.RemoveAll(this);
	FVisualLoggerDatabase::Get().GetEvents().OnItemSelectionChanged.RemoveAll(this);
}

void SVisualLoggerStatusView::OnExpansionChanged(TSharedPtr<FLogStatusItem> Item, bool bIsExpanded)
{
	if (bIsExpanded)
	{
		ExpandedCategories.AddUnique(Item->ItemText);
	}
	else
	{
		ExpandedCategories.RemoveSwap(Item->ItemText);
	}
}

void GenerateChildren(TSharedPtr<FLogStatusItem> StatusItem, const FVisualLogStatusCategory LogCategory)
{
	for (int32 LineIndex = 0; LineIndex < LogCategory.Data.Num(); LineIndex++)
	{
		FString KeyDesc, ValueDesc;
		const bool bHasValue = LogCategory.GetDesc(LineIndex, KeyDesc, ValueDesc);
		if (bHasValue)
		{
			StatusItem->Children.Add(MakeShareable(new FLogStatusItem(KeyDesc, ValueDesc)));
		}
	}

	for (const FVisualLogStatusCategory& Child : LogCategory.Children)
	{
		TSharedPtr<FLogStatusItem> ChildCategory = MakeShareable(new FLogStatusItem(Child.Category));
		StatusItem->Children.Add(ChildCategory);
		GenerateChildren(ChildCategory, Child);
	}
}

void SVisualLoggerStatusView::ResetData()
{
	StatusItems.Empty();
	StatusItemsView->RequestTreeRefresh();
}

void SVisualLoggerStatusView::OnObjectSelectionChanged(const TArray<FName>& SelectedItems)
{
	if (SelectedItems.Num() == 0)
	{
		ResetData();
	}
}

void SVisualLoggerStatusView::OnItemSelectionChanged(const FVisualLoggerDBRow& ChangedDBRow, int32 ItemIndex)
{
	StatusItems.Empty();
	StatusItemsView->RequestTreeRefresh();

	const TArray<FName>& SelectedRows = FVisualLoggerDatabase::Get().GetSelectedRows();
	for (auto RowName : SelectedRows)
	{
		const FVisualLoggerDBRow& CurrentDBRow = FVisualLoggerDatabase::Get().GetRowByName(RowName);
		if (FVisualLoggerDatabase::Get().IsRowVisible(RowName) == false || CurrentDBRow.GetCurrentItemIndex() == INDEX_NONE)
		{
			continue;
		}

		const TArray<FVisualLogDevice::FVisualLogEntryItem>& Entries = CurrentDBRow.GetItems();
		//int32 BestItemIndex = INDEX_NONE;
		//float BestDistance = MAX_FLT;
		//for (int32 Index = 0; Index < Entries.Num(); Index++)
		//{
		//	auto& CurrentEntryItem = Entries[Index];

		//	if (CurrentDBRow.IsItemVisible(Index) == false)
		//	{
		//		continue;
		//	}

		//	TArray<FVisualLoggerCategoryVerbosityPair> OutCategories;
		//	const float LastSelectedTimestep = CurrentDBRow.GetCurrentItemIndex() != INDEX_NONE ? CurrentDBRow.GetCurrentItem().Entry.TimeStamp : CurrentEntryItem.Entry.TimeStamp;
		//	const float CurrentDist = FMath::Abs(CurrentEntryItem.Entry.TimeStamp - LastSelectedTimestep);
		//	if (CurrentDist < BestDistance)
		//	{
		//		BestDistance = CurrentDist;
		//		BestItemIndex = Index;
		//	}
		//}
		

		//if (Entries.IsValidIndex(BestItemIndex))
		{
			//GenerateStatusData(Entries[BestItemIndex], SelectedRows.Num() > 1);
			GenerateStatusData(CurrentDBRow.GetCurrentItem(), SelectedRows.Num() > 1);
		}
	}

	{
		TUniqueFunction<void(TConstArrayView<TSharedPtr<FLogStatusItem>>)> ExpandCategoriesRecursively;
		ExpandCategoriesRecursively = [&](TConstArrayView< TSharedPtr<FLogStatusItem> > LogStatusItems)
		{
			for (const TSharedPtr<FLogStatusItem>& StatusItem : LogStatusItems)
			{
				if (ExpandedCategories.Contains(StatusItem->ItemText))
				{
					StatusItemsView->SetItemExpansion(StatusItem, true);
					ExpandCategoriesRecursively(StatusItem->Children);
				}
			}
		};
 
		ExpandCategoriesRecursively(StatusItems);
	}
}

void SVisualLoggerStatusView::GenerateStatusData(const FVisualLogDevice::FVisualLogEntryItem& LogEntry, bool bAddHeader)
{
	if (bAddHeader)
	{
		FLogStatusItem* HeaderItem = new FLogStatusItem();
		HeaderItem->HeaderText = FString::Printf(TEXT("%s at WorldTime: %.2lfs"), *LogEntry.OwnerName.ToString(), LogEntry.Entry.WorldTimeStamp);
		StatusItems.Add(MakeShareable(HeaderItem));
	}
	else
	{
		FString TimestampDesc = FString::Printf(TEXT("%.2lfs"), LogEntry.Entry.WorldTimeStamp);
		StatusItems.Add(MakeShareable(new FLogStatusItem(LOCTEXT("VisLogTimestamp", "WorldTime").ToString(), TimestampDesc)));
	}

	for (int32 CategoryIndex = 0; CategoryIndex < LogEntry.Entry.Status.Num(); CategoryIndex++)
	{
		if (LogEntry.Entry.Status[CategoryIndex].Data.Num() <= 0 && LogEntry.Entry.Status[CategoryIndex].Children.Num() == 0)
		{
			continue;
		}

		TSharedRef<FLogStatusItem> StatusItem = MakeShareable(new FLogStatusItem(LogEntry.Entry.Status[CategoryIndex].Category));
		for (int32 LineIndex = 0; LineIndex < LogEntry.Entry.Status[CategoryIndex].Data.Num(); LineIndex++)
		{
			FString KeyDesc, ValueDesc;
			const bool bHasValue = LogEntry.Entry.Status[CategoryIndex].GetDesc(LineIndex, KeyDesc, ValueDesc);
			if (bHasValue)
			{
				TSharedPtr< FLogStatusItem > NewItem = MakeShareable(new FLogStatusItem(KeyDesc, ValueDesc));
				StatusItem->Children.Add(NewItem);
			}
		}

		for (const FVisualLogStatusCategory& Child : LogEntry.Entry.Status[CategoryIndex].Children)
		{
			TSharedPtr<FLogStatusItem> ChildCategory = MakeShareable(new FLogStatusItem(Child.Category));
			StatusItem->Children.Add(ChildCategory);
			GenerateChildren(ChildCategory, Child);
		}

		StatusItems.Add(StatusItem);
	}

	StatusItemsView->RequestTreeRefresh();
}

void SVisualLoggerStatusView::OnLogStatusGetChildren(TSharedPtr<FLogStatusItem> InItem, TArray< TSharedPtr<FLogStatusItem> >& OutItems)
{
	OutItems = InItem->Children;
}

TSharedRef<ITableRow> SVisualLoggerStatusView::HandleGenerateLogStatus(TSharedPtr<FLogStatusItem> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	if (InItem->HeaderText.Len() > 0)
	{
		return SNew(STableRow<TSharedPtr<FLogStatusItem> >, OwnerTable)
			.Style(&FLogVisualizerStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.DarkRow"))
			[
				SNew(STextBlock)
				.Text(FText::FromString(InItem->HeaderText))
				.ColorAndOpacity(FColorList::LightGrey)
			];
	}
	
	if (InItem->Children.Num() > 0)
	{
		return SNew(STableRow<TSharedPtr<FLogStatusItem> >, OwnerTable)
			[
				SNew(STextBlock)
				.Text(FText::FromString(InItem->ItemText))
			];
	}

	FString TooltipText = FString::Printf(TEXT("%s: %s"), *InItem->ItemText, *InItem->ValueText);
	return SNew(STableRow<TSharedPtr<FLogStatusItem> >, OwnerTable)
		[
			SNew(SBorder)
			.BorderImage(FLogVisualizerStyle::Get().GetBrush("NoBorder"))
			.ToolTipText(FText::FromString(TooltipText))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(FText::FromString(InItem->ItemText))
					.ColorAndOpacity(FColorList::Aquamarine)
				]
				+ SHorizontalBox::Slot()
				.Padding(4.0f, 0, 0, 0)
				[
					SNew(STextBlock)
					.Text(FText::FromString(InItem->ValueText))
				]
			]
		];
}

#undef LOCTEXT_NAMESPACE
