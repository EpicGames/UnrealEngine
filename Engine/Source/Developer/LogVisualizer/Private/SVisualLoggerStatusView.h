// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Framework/Commands/UICommandList.h"
#include "VisualLogger/VisualLoggerTypes.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"
#include "SVisualLoggerBaseWidget.h"

struct FLogStatusItem;
struct FVisualLoggerDBRow;

class SVisualLoggerStatusView : public SVisualLoggerBaseWidget
{
public:
	SLATE_BEGIN_ARGS(SVisualLoggerStatusView){}
	SLATE_END_ARGS();

	virtual ~SVisualLoggerStatusView();
	void Construct(const FArguments& InArgs, const TSharedRef<FUICommandList>& InCommandList);
	void ResetData();

protected:
	void OnObjectSelectionChanged(const TArray<FName>& SelectedRows);
	void OnItemSelectionChanged(const FVisualLoggerDBRow& DBRow, int32 ItemIndex);
	void GenerateStatusData(const FVisualLogDevice::FVisualLogEntryItem&, bool bAddHeader);

	TSharedRef<ITableRow> HandleGenerateLogStatus(TSharedPtr<struct FLogStatusItem> InItem, const TSharedRef<STableViewBase>& OwnerTable);
	void OnLogStatusGetChildren(TSharedPtr<struct FLogStatusItem> InItem, TArray< TSharedPtr<FLogStatusItem> >& OutItems);
	void OnExpansionChanged(TSharedPtr<FLogStatusItem> Item, bool);

protected:
	TArray<FString> ExpandedCategories;
	TSharedPtr< STreeView< TSharedPtr<FLogStatusItem> > > StatusItemsView;
	TArray< TSharedPtr<FLogStatusItem> > StatusItems;
};
