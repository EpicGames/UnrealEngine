// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

/* Dependencies
*****************************************************************************/

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "SVisualLogger.h"
#include "Widgets/Docking/SDockTab.h"
#include "LogVisualizerPublic.h"

class SVisualLoggerTab : public SDockTab
{
public:
	virtual bool SupportsKeyboardFocus() const override { return true; }
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override
	{
		return FLogVisualizer::Get().GetEvents().OnKeyboardEvent.Execute(MyGeometry, InKeyEvent);
	}
};
