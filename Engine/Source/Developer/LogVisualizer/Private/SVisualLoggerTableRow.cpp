// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVisualLoggerTableRow.h"

const FLazyName SVisualLoggerTableRow::ColumnId_CategoryLabel("Category");
const FLazyName SVisualLoggerTableRow::ColumnId_VerbosityLabel("Verbosity");
const FLazyName SVisualLoggerTableRow::ColumnId_LogLabel("Log");
