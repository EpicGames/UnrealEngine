// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVisualLoggerTimeSlider.h"
#include "VisualLoggerTimeSliderController.h"

#define LOCTEXT_NAMESPACE "STimeSlider"


void SVisualLoggerTimeSlider::Construct( const SVisualLoggerTimeSlider::FArguments& InArgs, TSharedRef<FVisualLoggerTimeSliderController> InTimeSliderController )
{
	TimeSliderController = InTimeSliderController;
	bMirrorLabels = InArgs._MirrorLabels;
}

int32 SVisualLoggerTimeSlider::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	int32 NewLayer = TimeSliderController->OnPaintTimeSlider( bMirrorLabels, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled );

	return FMath::Max( NewLayer, SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, NewLayer, InWidgetStyle, ShouldBeEnabled( bParentEnabled ) ) );
}

FReply SVisualLoggerTimeSlider::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	return TimeSliderController->OnMouseButtonDown( *this, MyGeometry, MouseEvent );
}

FReply SVisualLoggerTimeSlider::OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	return TimeSliderController->OnMouseButtonUp( *this,  MyGeometry, MouseEvent );
}

FReply SVisualLoggerTimeSlider::OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	return TimeSliderController->OnMouseMove( *this, MyGeometry, MouseEvent );
}

FVector2D SVisualLoggerTimeSlider::ComputeDesiredSize( float ) const
{
	return FVector2D(100, 22);
}

FReply SVisualLoggerTimeSlider::OnMouseWheel( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	return TimeSliderController->OnMouseWheel( *this, MyGeometry, MouseEvent );
}

#undef LOCTEXT_NAMESPACE
