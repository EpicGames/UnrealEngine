// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVisualLoggerTimeline.h"

#if UE_DEBUG_VISUALIZER_TOOL_ENABLED
#include "LogVisualizerSettings.h"
#include "LogVisualizerStyle.h"
#include "Styling/CoreStyle.h"
#include "SVisualLoggerTimelineBar.h"
#include "SVisualLoggerTimelinesContainer.h"
#include "VisualLoggerDatabase.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"

class STimelineLabelAnchor : public SMenuAnchor
{
public:
	void Construct(const FArguments& InArgs, TSharedPtr<class SLogVisualizerTimeline> InTimelineOwner)
	{
		SMenuAnchor::Construct(InArgs);
		TimelineOwner = InTimelineOwner;
	}

private:
	/** SWidget Interface */
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	TWeakPtr<class SLogVisualizerTimeline> TimelineOwner;
};

FReply STimelineLabelAnchor::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton && TimelineOwner.IsValid() && TimelineOwner.Pin()->IsSelected())
	{
		SetIsOpen(!IsOpen());
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

FReply SLogVisualizerTimeline::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	Owner->ChangeSelection(SharedThis(this), MouseEvent);

	return FReply::Unhandled();
}

FReply SLogVisualizerTimeline::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	FReply ReturnValue = FReply::Unhandled();
	if (IsSelected() == false)
	{
		return ReturnValue;
	}

	return FLogVisualizer::Get().GetEvents().OnKeyboardEvent.Execute(MyGeometry, InKeyEvent);
}

bool SLogVisualizerTimeline::IsSelected() const
{
	// Ask the tree if we are selected
	return FVisualLoggerDatabase::Get().IsRowSelected(GetName());
}

void SLogVisualizerTimeline::UpdateVisibility()
{
	const ULogVisualizerSettings* Settings = GetDefault<ULogVisualizerSettings>();

	const FName RowName = GetName();
	if (FVisualLoggerDatabase::Get().ContainsRowByName(RowName))
	{
		FVisualLoggerDBRow& DataRow = FVisualLoggerDatabase::Get().GetRowByName(RowName);
		const TArray<FVisualLogDevice::FVisualLogEntryItem>& Entries = DataRow.GetItems();

		const bool bJustIgnore = Settings->bIgnoreTrivialLogs && Entries.Num() <= Settings->TrivialLogsThreshold;
		const bool bVisibleByOwnerClasses = FVisualLoggerFilters::Get().MatchObjectName(OwnerClassName.ToString());
		const bool bHiddenBySearchFilter = (SearchFilter.Len() > 0 && OwnerName.ToString().Find(SearchFilter) == INDEX_NONE);
		const bool bIsCollapsed = bJustIgnore || DataRow.GetNumberOfHiddenItems() == Entries.Num() || bHiddenBySearchFilter;
		DataRow.SetHiddenBySearchFilter(bHiddenBySearchFilter);
		const bool bIsHidden = bIsCollapsed || !bVisibleByOwnerClasses;
		SetVisibility(bIsHidden ? EVisibility::Collapsed : EVisibility::Visible);
		if (bIsCollapsed)
		{
			Owner->SetSelectionState(SharedThis(this), false, false);
		}

		FVisualLoggerDatabase::Get().SetRowVisibility(RowName, !bIsHidden);
	}
	else
	{
		Owner->SetSelectionState(SharedThis(this), false, false);
		FVisualLoggerDatabase::Get().SetRowVisibility(RowName, false);
	}
}

void SLogVisualizerTimeline::OnFiltersSearchChanged(const FText& Filter)
{
	OnFiltersChanged();
}

void SLogVisualizerTimeline::OnFiltersChanged()
{
	UpdateVisibility();
}

void SLogVisualizerTimeline::OnSearchChanged(const FText& Filter)
{
	SearchFilter = Filter.ToString();
	UpdateVisibility();
}

void SLogVisualizerTimeline::HandleLogVisualizerSettingChanged(FName InName)
{
	UpdateVisibility();
}

const TArray<FVisualLogDevice::FVisualLogEntryItem>& SLogVisualizerTimeline::GetEntries()
{
	FVisualLoggerDBRow& DataRow = FVisualLoggerDatabase::Get().GetRowByName(GetName());
	return DataRow.GetItems();
}

void SLogVisualizerTimeline::AddEntry(const FVisualLogDevice::FVisualLogEntryItem& Entry) 
{ 
	UpdateVisibility();
}

void SLogVisualizerTimeline::Construct(const FArguments& InArgs, TSharedPtr<FVisualLoggerTimeSliderController> InTimeSliderController, TSharedPtr<SVisualLoggerTimelinesContainer> InContainer, 
	const FName& InOwnerName, const FName& InOwnerDisplayName, const FName& InOwnerClassName)
{
	OnGetMenuContent = InArgs._OnGetMenuContent;

	Owner = InContainer;
	OwnerName = InOwnerName;
	OwnerDisplayName = InOwnerDisplayName;
	OwnerClassName = InOwnerClassName;
	
	ChildSlot
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(FMargin(0, 4, 0, 0))
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.FillWidth(TAttribute<float>::Create(TAttribute<float>::FGetter::CreateLambda([=] { return FLogVisualizer::Get().GetAnimationOutlinerFillPercentage(); })))
			[
				SAssignNew(PopupAnchor, STimelineLabelAnchor, SharedThis(this))
				.OnGetMenuContent(OnGetMenuContent)
				[
					SNew(SBorder)
					.HAlign(HAlign_Fill)
					.Padding(FMargin(0, 0, 4, 0))
					.BorderImage(FCoreStyle::Get().GetBrush("NoBorder"))
					[
						SNew(SBorder)
						.VAlign(VAlign_Center)
						.BorderImage(this, &SLogVisualizerTimeline::GetBorder)
						.Padding(FMargin(4, 0, 2, 0))
						[
							SNew(SVerticalBox)
								.ToolTip(
									SNew(SToolTip)
									.Text(FText::FromName(OwnerName))
								)
							+ SVerticalBox::Slot()
							.Padding(FMargin(0, 0, 0, 0))
							.HAlign(HAlign_Fill)
							.VAlign(VAlign_Fill)
							[
								SNew(STextBlock)
								.Text(FText::FromName(OwnerDisplayName))
								.ShadowOffset(FVector2D(1.f, 1.f))
							]
							+ SVerticalBox::Slot()
							.Padding(FMargin(0, 0, 0, 0))
							.HAlign(HAlign_Fill)
							.VAlign(VAlign_Fill)
							[
								SNew(STextBlock)
								.Text(FText::FromName(OwnerClassName))
								.TextStyle(FLogVisualizerStyle::Get(), TEXT("Sequencer.ClassName"))
							]
						]
					]
				]
			]
			+ SHorizontalBox::Slot()
			.Padding(FMargin(0, 4, 0, 0))
			.HAlign(HAlign_Left)
			[
				SNew(SBox)
				.Padding(FMargin(0, 0, 0, 0))
				.HAlign(HAlign_Left)
				[
					// Search box for searching through the outliner
					SAssignNew(TimelineBar, SVisualLoggerTimelineBar, InTimeSliderController, SharedThis(this))
				]
			]
		];

	GetMutableDefault<ULogVisualizerSettings>()->OnSettingChanged().AddRaw(this, &SLogVisualizerTimeline::HandleLogVisualizerSettingChanged);
	FVisualLoggerDatabase::Get().GetEvents().OnNewItem.AddRaw(this, &SLogVisualizerTimeline::OnNewItemHandler);
	FVisualLoggerDatabase::Get().GetEvents().OnRowSelectionChanged.AddRaw(this, &SLogVisualizerTimeline::OnRowSelectionChanged);
}

SLogVisualizerTimeline::~SLogVisualizerTimeline()
{
	GetMutableDefault<ULogVisualizerSettings>()->OnSettingChanged().RemoveAll(this);
	FVisualLoggerDatabase::Get().GetEvents().OnNewItem.RemoveAll(this);
	FVisualLoggerDatabase::Get().GetEvents().OnRowSelectionChanged.RemoveAll(this);
}

void SLogVisualizerTimeline::OnRowSelectionChanged(const TArray<FName>& RowNames)
{
	if (RowNames.Num() > 0)
	{
		const FName RowName = RowNames[RowNames.Num() - 1];
	}
}

void SLogVisualizerTimeline::OnNewItemHandler(const FVisualLoggerDBRow& BDRow, int32 ItemIndex)
{
	const FVisualLogDevice::FVisualLogEntryItem& Entry = BDRow.GetItems()[ItemIndex];
	if (GetName() == Entry.OwnerName)
	{
		AddEntry(Entry);
	}
}

const FSlateBrush* SLogVisualizerTimeline::GetBorder() const
{
	if (IsSelected())
	{
		return FLogVisualizerStyle::Get().GetBrush("ToolBar.Button.Hovered");
	}
	else
	{
		return FLogVisualizerStyle::Get().GetBrush("ToolBar.Button.Normal");
	}
}

// @todo: This function currently doesn't do anything!
void SLogVisualizerTimeline::Goto(float ScrubPosition)
{

}

// @todo: This function currently doesn't do anything!
void SLogVisualizerTimeline::GotoNextItem()
{
	FLogVisualizer::Get().GotoNextItem(GetName());
}

// @todo: This function currently doesn't do anything!
void SLogVisualizerTimeline::GotoPreviousItem()
{
	FLogVisualizer::Get().GotoPreviousItem(GetName());
}

// @todo: This function currently doesn't do anything!
void SLogVisualizerTimeline::MoveCursorByDistance(int32 Distance)
{
	if (Distance > 0)
	{
		FLogVisualizer::Get().GotoNextItem(GetName(), FMath::Abs(Distance));
	}
	else
	{
		FLogVisualizer::Get().GotoPreviousItem(GetName(), FMath::Abs(Distance));
	}
}
#endif // UE_DEBUG_VISUALIZER_TOOL_ENABLED