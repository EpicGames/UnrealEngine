// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VisualLogger/VisualLoggerDefines.h"

#if UE_DEBUG_VISUALIZER_TOOL_ENABLED
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "VisualLogger/VisualLoggerTypes.h"
#include "Framework/SlateDelegates.h"

class FVisualLoggerTimeSliderController;
class SMenuAnchor;
class SVisualLoggerTimelineBar;
class SVisualLoggerTimelinesContainer;
struct FSlateBrush;
struct FVisualLoggerDBRow;

/**
* A list of filters currently applied to an asset view.
*/
class SLogVisualizerTimeline : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SLogVisualizerTimeline){}
		SLATE_EVENT(FOnGetContent, OnGetMenuContent)
	SLATE_END_ARGS();

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual bool SupportsKeyboardFocus() const override { return true; }

	virtual TOptional<bool> OnQueryShowFocus(const EFocusCause InFocusCause) const override
	{
		return TOptional<bool>(false);
	}

	/** Constructs this widget with InArgs */
	virtual ~SLogVisualizerTimeline();
	void Construct(const FArguments& InArgs, TSharedPtr<class FVisualLoggerTimeSliderController> InTimeSliderController, TSharedPtr<class SVisualLoggerTimelinesContainer> InContainer, 
		const FName& InOwnerName, const FName& InOwnerDisplayName, const FName& InOwnerClassName);
	bool IsSelected() const;
	const FSlateBrush* GetBorder() const;

	void OnFiltersChanged();
	void OnSearchChanged(const FText& Filter);
	void OnFiltersSearchChanged(const FText& Filter);
	void UpdateVisibility();

	void AddEntry(const FVisualLogDevice::FVisualLogEntryItem& Entry);
	const TArray<FVisualLogDevice::FVisualLogEntryItem>& GetEntries();
	FName GetName() const { return OwnerName; }
	FName GetOwnerClassName() const { return OwnerClassName; }
	void HandleLogVisualizerSettingChanged(FName Name);
	TSharedPtr<class SVisualLoggerTimelinesContainer> GetOwner() { return Owner; }

	// @todo: This function currently doesn't do anything!
	void Goto(float ScrubPosition);
	// @todo: This function currently doesn't do anything!
	void GotoNextItem();
	// @todo: This function currently doesn't do anything!
	void GotoPreviousItem();
	// @todo: This function currently doesn't do anything!
	void MoveCursorByDistance(int32 Distance);

protected:
	void OnNewItemHandler(const FVisualLoggerDBRow& BDRow, int32 ItemIndex);
	void OnRowSelectionChanged(const TArray<FName>& RowNames);

protected:
	TSharedPtr<class SVisualLoggerTimelinesContainer> Owner;
	TSharedPtr<class SVisualLoggerTimelineBar> TimelineBar;
	TSharedPtr<SMenuAnchor> PopupAnchor;

	FName OwnerName;
	FName OwnerDisplayName;
	FName OwnerClassName;
	FString SearchFilter;

	/** Delegate to execute to get the menu content of this timeline */
	FOnGetContent OnGetMenuContent;
};
#endif // UE_DEBUG_VISUALIZER_TOOL_ENABLED