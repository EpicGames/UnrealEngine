// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVisualLoggerTimelineBar.h"
#if UE_DEBUG_VISUALIZER_TOOL_ENABLED
#include "Layout/ArrangedChildren.h"
#include "Rendering/DrawElements.h"
#include "VisualLoggerDatabase.h"
#include "LogVisualizerStyle.h"
#include "LogVisualizerPublic.h"
#include "LogVisualizerSettings.h"
#include "VisualLoggerTimeSliderController.h"
#include "Misc/OutputDeviceHelper.h"

FReply SVisualLoggerTimelineBar::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	TimelineOwner.Pin()->OnMouseButtonDown(MyGeometry, MouseEvent);
	FReply Reply = TimeSliderController->OnMouseButtonDown(*this, MyGeometry, MouseEvent);
	bool bHandleLeftMouseButton = MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton;

	// Only snap to closest item for the left button. We keep right button to pan.
	if (Reply.IsEventHandled() && bHandleLeftMouseButton)
	{
		FName RowName = TimelineOwner.Pin()->GetName();
		FVisualLoggerDBRow& DBRow = FVisualLoggerDatabase::Get().GetRowByName(RowName);
		const double ScrubPosition = TimeSliderController->GetTimeSliderArgs().ScrubPosition.Get();
		const int32 ClosestItem = DBRow.GetClosestItem(ScrubPosition);
		const auto& Items = DBRow.GetItems();
		if (Items.IsValidIndex(ClosestItem))
		{
			TimeSliderController->CommitScrubPosition(Items[ClosestItem].Entry.TimeStamp, false);
		}
	}
	return Reply;
}

FReply SVisualLoggerTimelineBar::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	TimelineOwner.Pin()->OnMouseButtonUp(MyGeometry, MouseEvent);

	// Only snap to closest item for the left button. We keep right button to pan.
	FReply Reply = TimeSliderController->OnMouseButtonUp(*this, MyGeometry, MouseEvent);
	bool bHandleLeftMouseButton = MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton;

	if (Reply.IsEventHandled() && bHandleLeftMouseButton)
	{
		FName RowName = TimelineOwner.Pin()->GetName();
		FVisualLoggerDBRow& DBRow = FVisualLoggerDatabase::Get().GetRowByName(RowName);
		const double ScrubPosition = TimeSliderController->GetTimeSliderArgs().ScrubPosition.Get();
		const int32 ClosestItem = DBRow.GetClosestItem(ScrubPosition);
		const auto& Items = DBRow.GetItems();
		if (Items.IsValidIndex(ClosestItem))
		{
			TimeSliderController->CommitScrubPosition(Items[ClosestItem].Entry.TimeStamp, false);
		}
	}
	return Reply;
}

FReply SVisualLoggerTimelineBar::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FName RowName = TimelineOwner.Pin()->GetName();
	FVisualLoggerDBRow& DBRow = FVisualLoggerDatabase::Get().GetRowByName(RowName);

	const double ClosestMouseTime = TimeSliderController->GetTimeAtCursorPosition(MyGeometry, MouseEvent);

	const int32 NewItemIndex = DBRow.GetClosestItem(ClosestMouseTime);
	const auto& Items = DBRow.GetItems();

	if (NewItemIndex != MouseMoveClosestItemIndex
		|| bToolTipUpdateRequested)
	{
		bToolTipUpdateRequested = false;
		MouseMoveClosestItemIndex = NewItemIndex;

		FString TooltipBuilder;
		if (Items.IsValidIndex(MouseMoveClosestItemIndex))
		{
			const FVisualLogEntry& CurrentEntry = Items[MouseMoveClosestItemIndex].Entry;
			
			TooltipBuilder = FString::Printf(TEXT("Time: %.2lf WorldTime: %.2lf"), CurrentEntry.TimeStamp, CurrentEntry.WorldTimeStamp);

			int32 DebugShapesWithoutTextCount = 0;
			for (const FVisualLogShapeElement& Shape : CurrentEntry.ElementsToDraw)
			{
				if (FVisualLoggerFilters::Get().ShouldDisplayCategory(Shape.Category, Shape.Verbosity))
				{
					if (Shape.Description.IsEmpty())
					{
						DebugShapesWithoutTextCount++;
					}
					else
					{
						TooltipBuilder += FString::Printf(TEXT("\n(shape) %s[%s]: %s"), *Shape.Category.ToString(), ::ToString(Shape.Verbosity), *Shape.Description);
					}
				}
			}

			if (DebugShapesWithoutTextCount > 0)
			{
				TooltipBuilder += FString::Printf(TEXT("\n%d shape(s) without description"), DebugShapesWithoutTextCount);
			}

			const bool bSearchInsideLogs = GetDefault<ULogVisualizerSettings>()->bSearchInsideLogs;
			for (const FVisualLogLine& Line : CurrentEntry.LogLines)
			{
				if (FVisualLoggerFilters::Get().ShouldDisplayLine(Line, bSearchInsideLogs))
				{
					TooltipBuilder += FString::Printf(TEXT("\n(log) %s: %s"), ::ToString(Line.Verbosity), *Line.Line);
				}
			}
		}

		SetToolTipText(FText::AsCultureInvariant(TooltipBuilder));
	}

	return TimeSliderController->OnMouseMove(*this, MyGeometry, MouseEvent);
}

FReply SVisualLoggerTimelineBar::OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.IsLeftControlDown() || MouseEvent.IsLeftShiftDown())
	{
		return TimeSliderController->OnMouseWheel(*this, MyGeometry, MouseEvent);
	}

	return FReply::Unhandled();
}

FReply SVisualLoggerTimelineBar::OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FName RowName = TimelineOwner.Pin()->GetName();
	FVisualLoggerDBRow& DBRow = FVisualLoggerDatabase::Get().GetRowByName(RowName);
	UWorld* World = FLogVisualizer::Get().GetWorld();
	if (World && MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		FLogVisualizer::Get().UpdateCameraPosition(RowName, DBRow.GetCurrentItemIndex());
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

SVisualLoggerTimelineBar::~SVisualLoggerTimelineBar()
{
	FLogVisualizer::Get().GetEvents().OnFiltersChanged.Remove(OnFiltersChangedDelegateHandle);
}

void SVisualLoggerTimelineBar::Construct(const FArguments& InArgs, TSharedPtr<FVisualLoggerTimeSliderController> InTimeSliderController, TSharedPtr<SLogVisualizerTimeline> InTimelineOwner)
{
	TimeSliderController = InTimeSliderController;
	TimelineOwner = InTimelineOwner;

	TRange<double> LocalViewRange = TimeSliderController->GetTimeSliderArgs().ViewRange.Get();

	MouseMoveClosestItemIndex = INDEX_NONE;

	// Listen for changes in filters to force refresh the tooltip text of the element closest to the current mouse position
	OnFiltersChangedDelegateHandle = FLogVisualizer::Get().GetEvents().OnFiltersChanged.AddSPLambda(this, [&bToolTipUpdateRequested = bToolTipUpdateRequested]
	{
		bToolTipUpdateRequested = true;
	});
}

FVector2D SVisualLoggerTimelineBar::ComputeDesiredSize(float) const
{
	return FVector2D(5000.0f, 20.0f);
}

int32 SVisualLoggerTimelineBar::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	//@TODO: Optimize it like it was with old LogVisualizer, to draw everything much faster (SebaK)
	int32 RetLayerId = LayerId;

	FArrangedChildren ArrangedChildren(EVisibility::Visible);
	ArrangeChildren(AllottedGeometry, ArrangedChildren);

	TRange<double> LocalViewRange = TimeSliderController->GetTimeSliderArgs().ViewRange.Get();
	double LocalScrubPosition = TimeSliderController->GetTimeSliderArgs().ScrubPosition.Get();

	double ViewRange = LocalViewRange.Size<double>();
	double PixelsPerInput = ViewRange > 0. ? AllottedGeometry.GetLocalSize().X / ViewRange : 0.;

	// Draw a region around the entire section area
	FSlateDrawElement::MakeBox(
		OutDrawElements,
		RetLayerId++,
		AllottedGeometry.ToPaintGeometry(),
		FLogVisualizerStyle::Get().GetBrush("Sequencer.SectionArea.Background"),
		ESlateDrawEffect::None,
		TimelineOwner.Pin()->IsSelected() ? FLinearColor(.2f, .2f, .2f, 0.5f) : FLinearColor(.1f, .1f, .1f, 0.5f)
		);

	const FSlateBrush* FillImage = FLogVisualizerStyle::Get().GetBrush("LogVisualizer.LogBar.EntryDefault");
	static const FColor CurrentTimeColor(140, 255, 255, 255);
	static const FColor ErrorTimeColor(255, 0, 0, 255);
	static const FColor WarningTimeColor(255, 255, 0, 255);
	static const FColor SelectedBarColor(255, 255, 255, 255);
	const FSlateBrush* SelectedFillImage = FLogVisualizerStyle::Get().GetBrush("LogVisualizer.LogBar.Selected");

	const ESlateDrawEffect DrawEffects = ESlateDrawEffect::None;// bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

	TArray<double> ErrorTimes;
	TArray<double> WarningTimes;
	int32 EntryIndex = 0;

	FVisualLoggerDBRow& DBRow = FVisualLoggerDatabase::Get().GetRowByName(TimelineOwner.Pin()->GetName());
	auto &Entries = DBRow.GetItems();
	while (EntryIndex < Entries.Num())
	{
		const FVisualLogEntry& Entry = Entries[EntryIndex].Entry;
		if (Entry.TimeStamp < LocalViewRange.GetLowerBoundValue() || Entry.TimeStamp > LocalViewRange.GetUpperBoundValue())
		{
			EntryIndex++;
			continue;
		}

		if (DBRow.IsItemVisible(EntryIndex)==false)
		{
			EntryIndex++;
			continue;
		}

		// find bar width, connect all contiguous bars to draw them as one geometry (rendering optimization)
		const double StartPos = (Entry.TimeStamp - LocalViewRange.GetLowerBoundValue()) * PixelsPerInput - 2;
		double EndPos = (Entry.TimeStamp - LocalViewRange.GetLowerBoundValue()) * PixelsPerInput + 2;
		int32 StartIndex = EntryIndex;
		const bool bSearchInsideLogs = GetDefault<ULogVisualizerSettings>()->bSearchInsideLogs;
		for (; StartIndex < Entries.Num(); ++StartIndex)
		{
			const FVisualLogEntry& CurrentEntry = Entries[StartIndex].Entry;
			if (CurrentEntry.TimeStamp < LocalViewRange.GetLowerBoundValue() || CurrentEntry.TimeStamp > LocalViewRange.GetUpperBoundValue())
			{
				break;
			}

			if (DBRow.IsItemVisible(StartIndex) == false)
			{
				continue;
			}

			const TArray<FVisualLogLine>& LogLines = CurrentEntry.LogLines;
			bool bAddedWarning = false;
			bool bAddedError = false;
			for (const FVisualLogLine& CurrentLine : LogLines)
			{
				if (CurrentLine.Verbosity <= ELogVerbosity::Error
					&& !bAddedError
					&& FVisualLoggerFilters::Get().ShouldDisplayLine(CurrentLine, bSearchInsideLogs))
				{
					ErrorTimes.AddUnique(CurrentEntry.TimeStamp);
					bAddedError = true;
				}
				else if (CurrentLine.Verbosity == ELogVerbosity::Warning
					&& !bAddedWarning
					&& FVisualLoggerFilters::Get().ShouldDisplayLine(CurrentLine, bSearchInsideLogs))
				{
					WarningTimes.AddUnique(CurrentEntry.TimeStamp);
					bAddedWarning = true;
				}
				if (bAddedError && bAddedWarning)
				{
					break;
				}
			}

			const double CurrentStartPos = (CurrentEntry.TimeStamp - LocalViewRange.GetLowerBoundValue()) * PixelsPerInput - 2;
			if (CurrentStartPos > EndPos)
			{
				break;
			}
			EndPos = (CurrentEntry.TimeStamp - LocalViewRange.GetLowerBoundValue()) * PixelsPerInput + 2;
		}

		if (EndPos - StartPos > 0)
		{
			const float BarWidth = (EndPos - StartPos);
			FSlateDrawElement::MakeBox(
				OutDrawElements,
				RetLayerId,
				AllottedGeometry.ToPaintGeometry(
				FVector2f(BarWidth, AllottedGeometry.GetLocalSize().Y),
				FSlateLayoutTransform(FVector2f(StartPos, 0.0f))),
				FillImage,
				DrawEffects,
				CurrentTimeColor
				);
		}
		EntryIndex = StartIndex;
	}

	constexpr double NoSelectionTime = -1;
	double SelectedTime = NoSelectionTime;
	if (TimelineOwner.Pin()->IsSelected() && DBRow.GetCurrentItemIndex() != INDEX_NONE)
	{
		const  FVisualLogDevice::FVisualLogEntryItem& HighlightedItemEntry = DBRow.GetCurrentItem();
		SelectedTime = HighlightedItemEntry.Entry.TimeStamp;
	}

	if (WarningTimes.Num())
	{
		RetLayerId++;
	}

	for (const double CurrentTime : WarningTimes)
	{
		const double LinePos = (CurrentTime - LocalViewRange.GetLowerBoundValue()) * PixelsPerInput;
		const float BoxWidth = SelectedTime == CurrentTime ? 10.f : 6.f;
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			RetLayerId,
			AllottedGeometry.ToPaintGeometry(
			FVector2f(BoxWidth, AllottedGeometry.GetLocalSize().Y),
			FSlateLayoutTransform(FVector2f(LinePos - (0.5f * BoxWidth), 0.0f))),
			FillImage,
			DrawEffects,
			WarningTimeColor
			);
	}

	if (ErrorTimes.Num())
	{
		RetLayerId++;
	}

	for (const double CurrentTime : ErrorTimes)
	{
		const double LinePos = (CurrentTime - LocalViewRange.GetLowerBoundValue()) * PixelsPerInput;
		const float BoxWidth = SelectedTime == CurrentTime ? 10.f : 6.f;
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			RetLayerId,
			AllottedGeometry.ToPaintGeometry(
			FVector2f(BoxWidth, AllottedGeometry.GetLocalSize().Y),
			FSlateLayoutTransform(FVector2f(LinePos - (0.5f * BoxWidth), 0.0f))),
			FillImage,
			DrawEffects,
			ErrorTimeColor
			);
	}

	if (SelectedTime != NoSelectionTime)
	{
		const double LinePos = (SelectedTime - LocalViewRange.GetLowerBoundValue()) * PixelsPerInput;
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			++RetLayerId,
			AllottedGeometry.ToPaintGeometry(
			FVector2f(4, AllottedGeometry.GetLocalSize().Y),
			FSlateLayoutTransform(FVector2f(LinePos - 2, 0.0f))),
			SelectedFillImage,
			ESlateDrawEffect::None,
			SelectedBarColor
			);
	}

	return RetLayerId;
}
#endif // UE_DEBUG_VISUALIZER_TOOL_ENABLED