// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VisualLogger/VisualLoggerDefines.h"

#if UE_DEBUG_VISUALIZER_TOOL_ENABLED
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SLeafWidget.h"
#include "SVisualLoggerTimeline.h"

class FPaintArgs;
class FSlateWindowElementList;
class FVisualLoggerTimeSliderController;

/**
* A list of filters currently applied to an asset view.
*/

class SVisualLoggerTimelineBar : public SLeafWidget
{
public:
	SLATE_BEGIN_ARGS(SVisualLoggerTimelineBar){}
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	virtual bool SupportsKeyboardFocus() const override { return false;  }

	virtual ~SVisualLoggerTimelineBar();
	void Construct(const FArguments& InArgs, TSharedPtr<FVisualLoggerTimeSliderController> InTimeSliderController, TSharedPtr<class SLogVisualizerTimeline> TimelineOwner);
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	virtual FVector2D ComputeDesiredSize(float) const override;

protected:
	TSharedPtr<class FVisualLoggerTimeSliderController> TimeSliderController;
	TWeakPtr<class SLogVisualizerTimeline> TimelineOwner;
	FDelegateHandle OnFiltersChangedDelegateHandle;
	int32 MouseMoveClosestItemIndex = INDEX_NONE;
	bool bToolTipUpdateRequested = false;
};
#endif // UE_DEBUG_VISUALIZER_TOOL_ENABLED