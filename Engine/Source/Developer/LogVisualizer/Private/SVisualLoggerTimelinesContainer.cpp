// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVisualLoggerTimelinesContainer.h"
#include "SVisualLoggerTimeline.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "VisualLoggerDatabase.h"
#include "LogVisualizerStyle.h"
#include "VisualLoggerTimeSliderController.h"
#include "SVisualLoggerReport.h"


#define LOCTEXT_NAMESPACE "STimelinesContainer"


TSharedRef<SWidget> SVisualLoggerTimelinesContainer::GetRightClickMenuContent()
{
	FMenuBuilder MenuBuilder(true, NULL);
	MenuBuilder.BeginSection("VisualLogReports", LOCTEXT("VisualLogReports", "VisualLog Reports"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("GenerateReport", "Generate  Report"),
			LOCTEXT("GenerateReportTooltip", "Generate report from Visual Log events."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SVisualLoggerTimelinesContainer::GenerateReport))
			);
	}
	MenuBuilder.EndSection();

	FDisplayMetrics DisplayMetrics;
	FSlateApplication::Get().GetCachedDisplayMetrics(DisplayMetrics);

	const FVector2D DisplaySize(
		DisplayMetrics.PrimaryDisplayWorkAreaRect.Right - DisplayMetrics.PrimaryDisplayWorkAreaRect.Left,
		DisplayMetrics.PrimaryDisplayWorkAreaRect.Bottom - DisplayMetrics.PrimaryDisplayWorkAreaRect.Top);

	return
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.MaxHeight(DisplaySize.Y * 0.9)
		[
			MenuBuilder.MakeWidget()
		];
}

void SVisualLoggerTimelinesContainer::SetSelectionState(TSharedPtr<SLogVisualizerTimeline> AffectedNode, bool bSelect, bool bDeselectOtherNodes)
{
	const FName RowName = AffectedNode->GetName();
	const bool bIsSelected = FVisualLoggerDatabase::Get().IsRowSelected(RowName);
	if (bSelect && (!bIsSelected || bDeselectOtherNodes))
	{
		FVisualLoggerDatabase::Get().SelectRow(RowName, bDeselectOtherNodes);
	}
	else if (!bSelect && bIsSelected && AffectedNode.IsValid())
	{
		FVisualLoggerDatabase::Get().DeselectRow(RowName);
	}
}


void SVisualLoggerTimelinesContainer::OnObjectSelectionChanged(const TArray<FName>& RowNames)
{
	CachedSelectedTimelines.Reset();
	for (TSharedPtr<SLogVisualizerTimeline>& Timeline : TimelineItems)
	{
		if (RowNames.Find(Timeline->GetName()) != INDEX_NONE)
		{
			CachedSelectedTimelines.Add(Timeline);
		}
	}

	if (CachedSelectedTimelines.Num() >= 1)
	{
		FSlateApplication::Get().SetKeyboardFocus(SharedThis(CachedSelectedTimelines[CachedSelectedTimelines.Num()-1].Get()), EFocusCause::Navigation);
	}
}

bool SVisualLoggerTimelinesContainer::IsNodeSelected(TSharedPtr<SLogVisualizerTimeline> Node) const
{
	return FVisualLoggerDatabase::Get().IsRowSelected(Node->GetName());// SelectedNodes.Contains(Node);
}

void SVisualLoggerTimelinesContainer::ChangeSelection(TSharedPtr<SLogVisualizerTimeline> InTimeline, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.IsLeftShiftDown() == false)
	{
		if (MouseEvent.IsLeftControlDown())
		{
			SetSelectionState(InTimeline, !InTimeline->IsSelected(), false);
		}
		else
		{
			SetSelectionState(InTimeline, true, true);
		}
	}
	else
	{
		if (CachedSelectedTimelines.Num() == 0 && TimelineItems.Num())
		{
			SetSelectionState(TimelineItems[0], true, true);
		}

		TSharedPtr<SLogVisualizerTimeline> LastSelected = CachedSelectedTimelines.Num() ? CachedSelectedTimelines[CachedSelectedTimelines.Num() - 1] : nullptr;
		if (LastSelected.IsValid())
		{
			bool bStartedSelection = false;
			for (TSharedPtr<SLogVisualizerTimeline>& TimelineItem : TimelineItems)
			{
				if (TimelineItem == LastSelected || InTimeline == TimelineItem)
				{
					if (!bStartedSelection)
					{
						bStartedSelection = true;
					}
					else
					{
						bStartedSelection = false;
						break;
					}
				}
				if (bStartedSelection)
				{
					SetSelectionState(TimelineItem, true, false);
				}
			}
		}
		SetSelectionState(InTimeline, true, false);
	}
}

FReply SVisualLoggerTimelinesContainer::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() != EKeys::LeftMouseButton)
	{
		return TimeSliderController->OnMouseButtonDown(*this, MyGeometry, MouseEvent);
	}
	return FReply::Unhandled();
}

FReply SVisualLoggerTimelinesContainer::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() != EKeys::LeftMouseButton)
	{
		return TimeSliderController->OnMouseButtonUp(*this, MyGeometry, MouseEvent);
	}
	return FReply::Unhandled();
}

FReply SVisualLoggerTimelinesContainer::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() != EKeys::LeftMouseButton)
	{
		return TimeSliderController->OnMouseMove(*this, MyGeometry, MouseEvent);
	}
	return FReply::Unhandled();
}

FReply SVisualLoggerTimelinesContainer::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::A && InKeyEvent.IsLeftControlDown())
	{
		for (TSharedPtr<SLogVisualizerTimeline>& Timeline : TimelineItems)
		{
			SetSelectionState(Timeline, true, false);
		}

		return FReply::Handled();
	}
	else if (InKeyEvent.GetKey() == EKeys::Platform_Delete && CachedSelectedTimelines.Num() > 0)
	{
		TWeakPtr<SLogVisualizerTimeline>  NotSelectedOne;
		for (TSharedPtr<SLogVisualizerTimeline>& CurrentNode : CachedSelectedTimelines)
		{
			TSharedPtr<SLogVisualizerTimeline> LastSelected = CachedSelectedTimelines[CachedSelectedTimelines.Num() - 1];
			bool bFoundSelectedOne = false;

			for (TSharedPtr<SLogVisualizerTimeline>& Timeline : TimelineItems)
			{
				if (IsNodeSelected(Timeline) == false)
				{
					NotSelectedOne = Timeline;
				}
				if (LastSelected == Timeline)
				{
					if (bFoundSelectedOne && NotSelectedOne.IsValid())
					{
						break;
					}
					bFoundSelectedOne = true;
				}
			}
			FVisualLoggerDatabase::Get().RemoveRow(CurrentNode->GetName());
			TimelineItems.Remove(CurrentNode);
			ContainingBorder->RemoveSlot(CurrentNode.ToSharedRef());
		}

		if (NotSelectedOne.IsValid())
		{
			SetSelectionState(NotSelectedOne.Pin(), true, true);
		}

		return FReply::Handled();
	}
	else if (InKeyEvent.GetKey() == EKeys::Up || InKeyEvent.GetKey() == EKeys::Down)
	{
		TSharedPtr<SLogVisualizerTimeline> PreviousTimeline;
		TSharedPtr<SLogVisualizerTimeline> LastSelected = CachedSelectedTimelines[CachedSelectedTimelines.Num() - 1];
		for (int32 Index = 0; Index < TimelineItems.Num(); ++Index)
		{
			auto& CurrentItem = TimelineItems[Index];
			if (LastSelected == CurrentItem)
			{
				if (InKeyEvent.GetKey() == EKeys::Up && PreviousTimeline.IsValid())
				{
					SetSelectionState(PreviousTimeline, true, true);
				}
				else if (InKeyEvent.GetKey() == EKeys::Down)
				{
					// let's find next visible time line
					if (TimelineItems.IsValidIndex(Index + 1))
					{
						for (int32 i = Index + 1; i < TimelineItems.Num(); ++i)
						{
							if (TimelineItems[i]->GetVisibility() == EVisibility::Visible)
							{
								SetSelectionState(TimelineItems[i], true, true);
								break;
							}
						}
					}
				}
				break;
			}

			if (CurrentItem->GetVisibility() == EVisibility::Visible)
			{
				PreviousTimeline = CurrentItem;
			}
		}
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SVisualLoggerTimelinesContainer::ResetData()
{
	for (TSharedPtr<SLogVisualizerTimeline>& Timeline : TimelineItems)
	{
		ContainingBorder->RemoveSlot(Timeline.ToSharedRef());
		Timeline.Reset();
	}
	TimelineItems.Reset();

	for (auto& CurrentItem : CachedSelectedTimelines)
	{
		CurrentItem.Reset();
	}
	CachedSelectedTimelines.Reset();

	CachedMinTime = MAX_dbl;
	CachedMaxTime = 0.;
	TimeSliderController->SetClampRange(0, 5);
	TimeSliderController->SetTimeRange(0, 5);
}

void SVisualLoggerTimelinesContainer::Construct(const FArguments& InArgs, TSharedRef<SVisualLoggerView> InVisualLoggerView, TSharedRef<FVisualLoggerTimeSliderController> InTimeSliderController)
{
	TimeSliderController = InTimeSliderController;
	VisualLoggerView = InVisualLoggerView;

	ChildSlot
		[
			SNew(SBorder)
			.Padding(0)
			.VAlign(VAlign_Top)
			.BorderImage(FLogVisualizerStyle::Get().GetBrush("ToolPanel.GroupBorder"))
			[
				SAssignNew(ContainingBorder, SVerticalBox)
			]
		];

	CachedMinTime = MAX_dbl;
	CachedMaxTime = 0.;

	FVisualLoggerDatabase::Get().GetEvents().OnNewRow.AddRaw(this, &SVisualLoggerTimelinesContainer::OnNewRowHandler);
	FVisualLoggerDatabase::Get().GetEvents().OnNewItem.AddRaw(this, &SVisualLoggerTimelinesContainer::OnNewItemHandler);
	FVisualLoggerDatabase::Get().GetEvents().OnRowSelectionChanged.AddRaw(this, &SVisualLoggerTimelinesContainer::OnObjectSelectionChanged);
	FVisualLoggerDatabase::Get().GetEvents().OnRowChangedVisibility.AddRaw(this, &SVisualLoggerTimelinesContainer::OnRowChangedVisibility);
}

SVisualLoggerTimelinesContainer::~SVisualLoggerTimelinesContainer()
{
	FVisualLoggerDatabase::Get().GetEvents().OnNewRow.RemoveAll(this);
	FVisualLoggerDatabase::Get().GetEvents().OnNewItem.RemoveAll(this);
	FVisualLoggerDatabase::Get().GetEvents().OnRowSelectionChanged.RemoveAll(this);
	FVisualLoggerDatabase::Get().GetEvents().OnRowChangedVisibility.RemoveAll(this);
}

void SVisualLoggerTimelinesContainer::OnNewRowHandler(const FVisualLoggerDBRow& DBRow)
{
	TSharedPtr<SLogVisualizerTimeline> NewTimeline;

	ContainingBorder->AddSlot()
		[
			SAssignNew(NewTimeline, SLogVisualizerTimeline, TimeSliderController, SharedThis(this), DBRow.GetOwnerName(), DBRow.GetOwnerDisplayName(), DBRow.GetOwnerClassName())
			.OnGetMenuContent(this, &SVisualLoggerTimelinesContainer::GetRightClickMenuContent)
		];

	TimelineItems.Add(NewTimeline.ToSharedRef());
	// make sure the new entry doesn't show if it doesn't match CurrentSearchText
	NewTimeline->OnSearchChanged(CurrentSearchText);
}

void SVisualLoggerTimelinesContainer::OnNewItemHandler(const FVisualLoggerDBRow& BDRow, int32 ItemIndex)
{
	const FVisualLogDevice::FVisualLogEntryItem& Entry = BDRow.GetItems()[ItemIndex];
	CachedMinTime = CachedMinTime < Entry.Entry.TimeStamp ? CachedMinTime : Entry.Entry.TimeStamp;
	CachedMaxTime = CachedMaxTime > Entry.Entry.TimeStamp ? CachedMaxTime : Entry.Entry.TimeStamp;

	TRange<double> LocalViewRange = TimeSliderController->GetTimeSliderArgs().ViewRange.Get();
	TRange<double> ClampRange = TimeSliderController->GetTimeSliderArgs().ClampRange.Get();
	double ZoomLevel = LocalViewRange.Size<double>() / ClampRange.Size<double>();

	TimeSliderController->GetTimeSliderArgs().ClampRange = TRange<double>(CachedMinTime, CachedMaxTime + 0.1);
	TimeSliderController->GetTimeSliderArgs().ViewRange = TimeSliderController->GetTimeSliderArgs().ClampRange;
}

void SVisualLoggerTimelinesContainer::OnSearchChanged(const FText& Filter)
{
	CurrentSearchText = Filter;
	for (TSharedPtr<SLogVisualizerTimeline>& Timeline : TimelineItems)
	{
		Timeline->OnSearchChanged(Filter);
	}
}

void SVisualLoggerTimelinesContainer::OnFiltersChanged()
{
	for (TSharedPtr<SLogVisualizerTimeline>& Timeline : TimelineItems)
	{
		Timeline->OnFiltersChanged();
	}
}

void SVisualLoggerTimelinesContainer::OnFiltersSearchChanged(const FText& Filter)
{
	for (TSharedPtr<SLogVisualizerTimeline>& Timeline : TimelineItems)
	{
		Timeline->OnFiltersSearchChanged(Filter);
	}
}

void SVisualLoggerTimelinesContainer::GenerateReport()
{
	TSharedRef<SWindow> NewWindow = SNew(SWindow)
		.ClientSize(FVector2f(720.f, 768.f))
		.Title(NSLOCTEXT("LogVisualizerReport", "WindowTitle", "Log Visualizer Report"))
		[
			SNew(SVisualLoggerReport, CachedSelectedTimelines, VisualLoggerView)
		];

	FSlateApplication::Get().AddWindow(NewWindow);
}


void SVisualLoggerTimelinesContainer::OnRowChangedVisibility(const FName& InName)
{
	for (TSharedPtr<SLogVisualizerTimeline>& Timeline : TimelineItems)
	{
		if (InName == Timeline->GetName())
		{
			Timeline->SetVisibility(FVisualLoggerDatabase::Get().IsRowVisible(InName) ? EVisibility::Visible : EVisibility::Collapsed);
			break;
		}
	}
	
}

#undef LOCTEXT_NAMESPACE
