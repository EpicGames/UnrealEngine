// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "SVisualLoggerView.h"

class FVisualLoggerTimeSliderController;
class SLogVisualizerTimeline;
class STimelinesBar;
struct FVisualLoggerDBRow;

/**
* A list of filters currently applied to an asset view.
*/
class SVisualLoggerTimelinesContainer : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SVisualLoggerTimelinesContainer){}
	SLATE_END_ARGS()

	virtual ~SVisualLoggerTimelinesContainer();

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, TSharedRef<SVisualLoggerView>, TSharedRef<FVisualLoggerTimeSliderController> TimeSliderController);
	TSharedRef<SWidget> GetRightClickMenuContent();

	void OnTimelineSelected(TSharedPtr<STimelinesBar> Widget);
	void ChangeSelection(TSharedPtr<SLogVisualizerTimeline>, const FPointerEvent& MouseEvent);
	virtual bool SupportsKeyboardFocus() const override { return true; }

	void OnFiltersChanged();
	void OnSearchChanged(const FText& Filter);
	void OnFiltersSearchChanged(const FText& Filter);

	void ResetData();

	void GenerateReport();

	/**
	* Selects a node in the tree
	*
	* @param AffectedNode			The node to select
	* @param bSelect				Whether or not to select
	* @param bDeselectOtherNodes	Whether or not to deselect all other nodes
	*/
	void SetSelectionState(TSharedPtr<SLogVisualizerTimeline> AffectedNode, bool bSelect, bool bDeselectOtherNodes = true);

	/**
	* @return All currently selected nodes
	*/
	const TArray< TSharedPtr<SLogVisualizerTimeline> >& GetSelectedNodes() const { return CachedSelectedTimelines; }

	const TArray< TSharedPtr<SLogVisualizerTimeline> >& GetAllNodes() const { return TimelineItems; }

	/**
	* Returns whether or not a node is selected
	*
	* @param Node	The node to check for selection
	* @return true if the node is selected
	*/
	bool IsNodeSelected(TSharedPtr<SLogVisualizerTimeline> Node) const;

protected:
	void OnNewRowHandler(const FVisualLoggerDBRow& DBRow);
	void OnNewItemHandler(const FVisualLoggerDBRow& BDRow, int32 ItemIndex);
	void OnObjectSelectionChanged(const TArray<FName>& RowNames);
	void OnRowChangedVisibility(const FName&);

protected:
	TSharedPtr<SVerticalBox> ContainingBorder;
	TSharedPtr<FVisualLoggerTimeSliderController> TimeSliderController;
	TSharedPtr<SVisualLoggerView> VisualLoggerView;

	TArray<TSharedPtr<SLogVisualizerTimeline> > TimelineItems;
	TArray<TSharedPtr<SLogVisualizerTimeline> > CachedSelectedTimelines;
	double CachedMinTime;
	double CachedMaxTime;

	FText CurrentSearchText;
};
