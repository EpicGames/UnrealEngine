// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVisualLoggerToolbar.h"

#if UE_DEBUG_VISUALIZER_TOOL_ENABLED
#include "Styling/AppStyle.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "LogVisualizerStyle.h"
#include "LogVisualizerPublic.h"
#include "VisualLoggerCommands.h"
#include "VisualLoggerRenderingActor.h"
#include "VisualLoggerDatabase.h"
#define LOCTEXT_NAMESPACE "SVisualLoggerToolbar"

/* SVisualLoggerToolbar interface
*****************************************************************************/
void SVisualLoggerToolbar::Construct(const FArguments& InArgs, const TSharedRef<FUICommandList>& InCommandList)
{
	ChildSlot
		[
			MakeToolbar(InCommandList)
		];
}


/* SVisualLoggerToolbar implementation
*****************************************************************************/
TSharedRef<SWidget> SVisualLoggerToolbar::MakeToolbar(const TSharedRef<FUICommandList>& CommandList)
{
	FSlimHorizontalToolBarBuilder ToolBarBuilder(CommandList, FMultiBoxCustomization::None);

	ToolBarBuilder.BeginSection("Debugger");
	{
		ToolBarBuilder.AddToolBarButton(FVisualLoggerCommands::Get().StartRecording, NAME_None, LOCTEXT("StartLogger", "Start"), LOCTEXT("StartDebuggerTooltip", "Starts recording and collecting visual logs"), FSlateIcon(FLogVisualizerStyle::Get().GetStyleSetName(), TEXT("Toolbar.Record")));
		ToolBarBuilder.AddToolBarButton(FVisualLoggerCommands::Get().StopRecording, NAME_None, LOCTEXT("StopLogger", "Stop"), TAttribute<FText>(), FSlateIcon(FLogVisualizerStyle::Get().GetStyleSetName(), TEXT("Toolbar.Stop")));
		ToolBarBuilder.AddToolBarButton(FVisualLoggerCommands::Get().Pause, NAME_None, LOCTEXT("PauseLogger", "Pause"), TAttribute<FText>(), FSlateIcon(FLogVisualizerStyle::Get().GetStyleSetName(), TEXT("Toolbar.Pause")));
		ToolBarBuilder.AddToolBarButton(FVisualLoggerCommands::Get().Resume, NAME_None, LOCTEXT("ResumeLogger", "Resume"), TAttribute<FText>(), FSlateIcon(FLogVisualizerStyle::Get().GetStyleSetName(), TEXT("Toolbar.Resume")));

		ToolBarBuilder.AddSeparator();
		ToolBarBuilder.AddToolBarButton(FVisualLoggerCommands::Get().LoadFromVLog, NAME_None, LOCTEXT("Load", "Load"), TAttribute<FText>(), FSlateIcon(FLogVisualizerStyle::Get().GetStyleSetName(), TEXT("Toolbar.Load")));
		ToolBarBuilder.AddToolBarButton(FVisualLoggerCommands::Get().SaveToVLog, NAME_None, LOCTEXT("SaveLogs", "Save"), LOCTEXT("SaveLogsTooltip", "Save selected logs/rows to file."), FSlateIcon(FLogVisualizerStyle::Get().GetStyleSetName(), TEXT("Toolbar.Save")));
		ToolBarBuilder.AddToolBarButton(FVisualLoggerCommands::Get().SaveAllToVLog, NAME_None, LOCTEXT("SaveAllLogs", "Save All"), TAttribute<FText>(), FSlateIcon(FLogVisualizerStyle::Get().GetStyleSetName(), TEXT("Toolbar.SaveAll")));

		ToolBarBuilder.AddSeparator();
		ToolBarBuilder.AddToolBarButton(FVisualLoggerCommands::Get().FreeCamera, NAME_None, LOCTEXT("FreeCamera", "Camera"), TAttribute<FText>(), FSlateIcon(FLogVisualizerStyle::Get().GetStyleSetName(), TEXT("Toolbar.Camera")));
		ToolBarBuilder.AddToolBarButton(FVisualLoggerCommands::Get().ResetData, NAME_None, LOCTEXT("ResetData", "Clear"), TAttribute<FText>(), FSlateIcon(FLogVisualizerStyle::Get().GetStyleSetName(), TEXT("Toolbar.Remove")));
		ToolBarBuilder.AddToolBarButton(FVisualLoggerCommands::Get().ToggleGraphs, NAME_None, LOCTEXT("ToggleGraphs", "Graphs"), TAttribute<FText>(), FSlateIcon(FLogVisualizerStyle::Get().GetStyleSetName(), TEXT("Toolbar.Graphs")));
		ToolBarBuilder.AddToolBarButton(FVisualLoggerCommands::Get().Refresh, NAME_None, LOCTEXT("ForceRefresh", "Refresh"), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Refresh"));
		ToolBarBuilder.AddToolBarButton(FVisualLoggerCommands::Get().AutoScroll, NAME_None, LOCTEXT("AutoScroll", "Auto-Scroll"), TAttribute<FText>(), FSlateIcon(FLogVisualizerStyle::Get().GetStyleSetName(), "Toolbar.AutoScroll"));

		ToolBarBuilder.AddSeparator();
		ToolBarBuilder.AddToolBarWidget(MakeClippingDistanceWidget(), LOCTEXT("FarClippingDistance", "Far Clipping Distance"), /*InTutorialHighlightName = */NAME_None, /*bInSearchable = */true, LOCTEXT("FarClipDistanceTooltip", "Max distance after which visual log items will stop being rendered (ignored if <=0). This can help performance/visibility in situations where lots of visual logs are displayed on screen."));
	}

	return ToolBarBuilder.MakeWidget();
}

TSharedRef<SWidget> SVisualLoggerToolbar::MakeClippingDistanceWidget()
{
	return SNew(SSpinBox<double>)
		.MinValue(0.0)
		.Delta(10.0f)
		.MinDesiredWidth(70.0f)
		.Justification(ETextJustify::Center)
		.Value_Lambda([]()
			{
				AVisualLoggerRenderingActor* HelperActor = Cast<AVisualLoggerRenderingActor>(FVisualLoggerEditorInterface::Get()->GetHelperActor(FLogVisualizer::Get().GetWorld()));
				if (ensure(HelperActor != nullptr))
				{
					return HelperActor->GetFarClippingDistance();
				}
				return 0.0;
			})
		.OnValueChanged_Lambda([](double Value)
			{
				AVisualLoggerRenderingActor* HelperActor = Cast<AVisualLoggerRenderingActor>(FVisualLoggerEditorInterface::Get()->GetHelperActor(FLogVisualizer::Get().GetWorld()));
				if (ensure(HelperActor != nullptr))
				{
					HelperActor->SetFarClippingDistance(Value);
				}
			});
}

#undef LOCTEXT_NAMESPACE
#endif // UE_DEBUG_VISUALIZER_TOOL_ENABLED
