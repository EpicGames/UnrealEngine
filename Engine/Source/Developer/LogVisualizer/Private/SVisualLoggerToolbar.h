// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VisualLogger/VisualLoggerDefines.h"

#if UE_DEBUG_VISUALIZER_TOOL_ENABLED
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Framework/Commands/UICommandList.h"
#include "SVisualLoggerBaseWidget.h"

/**
* Implements the messaging debugger toolbar widget.
*/
class SVisualLoggerToolbar : public SVisualLoggerBaseWidget
{
public:

	SLATE_BEGIN_ARGS(SVisualLoggerToolbar) { }
	SLATE_END_ARGS()

public:

	/**
	* Construct this widget
	*
	* @param InArgs The declaration data for this widget.
	* @param InStyle The visual style to use for this widget.
	* @param InCommandList The command list to bind to.
	*/
	void Construct(const FArguments& InArgs, const TSharedRef<FUICommandList>& InCommandList);

protected:

	/**
	* Creates the toolbar widget.
	*
	* @param CommandList The command list to use for the toolbar buttons.
	* @return The toolbar widget.
	*/
	TSharedRef<SWidget> MakeToolbar(const TSharedRef<FUICommandList>& CommandList);

	TSharedRef<SWidget> MakeClippingDistanceWidget();
};
#endif // UE_DEBUG_VISUALIZER_TOOL_ENABLED