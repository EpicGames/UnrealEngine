// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVisualLoggerView.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "LogVisualizerSettings.h"
#include "LogVisualizerStyle.h"
#include "SVisualLoggerSectionOverlay.h"
#include "SVisualLoggerTimeline.h"
#include "SVisualLoggerTimelinesContainer.h"
#include "SVisualLoggerTimeSlider.h"
#include "VisualLoggerTimeSliderController.h"
#include "Widgets/Input/SSearchBox.h"


#define LOCTEXT_NAMESPACE "SVisualLoggerFilters"


class SInputCatcherOverlay : public SOverlay
{
public:
	void Construct(const FArguments& InArgs, TSharedRef<class FVisualLoggerTimeSliderController> InTimeSliderController)
	{
		SOverlay::Construct(InArgs);
		TimeSliderController = InTimeSliderController;
	}

	/** Controller for manipulating time */
	TSharedPtr<class FVisualLoggerTimeSliderController> TimeSliderController;
private:
	/** SWidget Interface */
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

private:
};

FReply SInputCatcherOverlay::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() != EKeys::LeftMouseButton)
	{
		return TimeSliderController->OnMouseButtonDown(*this, MyGeometry, MouseEvent);
	}
	return FReply::Unhandled();
}

FReply SInputCatcherOverlay::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() != EKeys::LeftMouseButton)
	{
		return TimeSliderController->OnMouseButtonUp(*this, MyGeometry, MouseEvent);
	}
	return FReply::Unhandled();
}

FReply SInputCatcherOverlay::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return TimeSliderController->OnMouseMove(*this, MyGeometry, MouseEvent);
}

FReply SInputCatcherOverlay::OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.IsLeftShiftDown() || MouseEvent.IsLeftControlDown())
	{
		return TimeSliderController->OnMouseWheel(*this, MyGeometry, MouseEvent);
	}
	return FReply::Unhandled();
}

void SVisualLoggerView::Construct(const FArguments& InArgs, const TSharedRef<FUICommandList>& InCommandList)
{
	AnimationOutlinerFillPercentage = .25f;

	TSharedRef<SScrollBar> ZoomScrollBar =
		SNew(SScrollBar)
		.Orientation(EOrientation::Orient_Horizontal)
		.Thickness(FVector2D(6.0f, 6.0f));
	ZoomScrollBar->SetState(0.0f, 1.0f);
	FLogVisualizer::Get().GetTimeSliderController()->SetExternalScrollbar(ZoomScrollBar);

	// Create the top and bottom sliders
	const bool bMirrorLabels = true;
	TSharedRef<SVisualLoggerTimeSlider> TopTimeSlider = SNew(SVisualLoggerTimeSlider, FLogVisualizer::Get().GetTimeSliderController().ToSharedRef()).MirrorLabels(bMirrorLabels);
	TSharedRef<SVisualLoggerTimeSlider> BottomTimeSlider = SNew(SVisualLoggerTimeSlider, FLogVisualizer::Get().GetTimeSliderController().ToSharedRef()).MirrorLabels(bMirrorLabels);

	TSharedRef<SScrollBar> ScrollBar =
		SNew(SScrollBar)
		.Thickness(FVector2D(6.0f, 6.0f));


	const ULogVisualizerSettings* Settings = GetDefault<ULogVisualizerSettings>();

	ChildSlot
		[
			SNew(SBorder)
			.Padding(2.f)
			.BorderImage(FLogVisualizerStyle::Get().GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SAssignNew(SearchSplitter, SSplitter)
						.Orientation(Orient_Horizontal)
						.OnSplitterFinishedResizing(this, &SVisualLoggerView::OnSearchSplitterResized)
						
						+ SSplitter::Slot()
						.Value(0.25)
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.Padding(FMargin(0))
							.AutoWidth()
							.HAlign(HAlign_Center)
							.VAlign(VAlign_Center)
							[
								SNew(SImage)
								.Visibility_Lambda([]()->EVisibility{ return FVisualLoggerFilters::Get().GetSelectedObjects().Num() > 0 ? EVisibility::Visible : EVisibility::Collapsed; })
								.Image(FLogVisualizerStyle::Get().GetBrush("Filters.FilterIcon"))
							]
							+ SHorizontalBox::Slot()
							.Padding(FMargin(0))
							.HAlign(HAlign_Right)
							.AutoWidth()
							[
								SAssignNew(ClassesComboButton, SComboButton)
								.Visibility_Lambda([this]()->EVisibility{ return TimelinesContainer.IsValid() && (TimelinesContainer->GetAllNodes().Num() > 1 || FVisualLoggerFilters::Get().GetSelectedObjects().Num() > 0) ? EVisibility::Visible : EVisibility::Collapsed; })
								.ComboButtonStyle(FLogVisualizerStyle::Get(), "Filters.Style")
								.ForegroundColor(FLinearColor::White)
								.ContentPadding(0.f)
								.OnGetMenuContent(this, &SVisualLoggerView::MakeClassesFilterMenu)
								.ToolTipText(LOCTEXT("SetFilterByClasses", "Select classes to show"))
								.HasDownArrow(true)
								.ContentPadding(FMargin(1, 0))
								.ButtonContent()
								[
									SNew(STextBlock)
									.TextStyle(FLogVisualizerStyle::Get(), "GenericFilters.TextStyle")
									.Text(LOCTEXT("FilterClasses", "Classes"))
								]
							]
							+ SHorizontalBox::Slot()
							.Padding(FMargin(0))
							.HAlign(HAlign_Fill)
							.FillWidth(1)
							[
								SNew(SBox)
								.Padding(FMargin(0, 0, 4, 0))
								[
									// Search box for searching through the outliner
									SNew(SSearchBox)
									.OnTextChanged(this, &SVisualLoggerView::OnSearchChanged)
								]
							]
						]
						+ SSplitter::Slot()
						.Value(0.75)
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							[
#if 0 //top time slider disabled to test idea with filter's search box
								SNew(SBorder)
								.Padding(FMargin(0.0f, 2.0f, 0.0f, 0.0f))
								.BorderImage(FLogVisualizerStyle::Get().GetBrush("ToolPanel.GroupBorder"))
								.BorderBackgroundColor(FLinearColor(.50f, .50f, .50f, 1.0f))
								[
									TopTimeSlider
								]
#else
								SNew(SBox)
								.Padding(FMargin(0, 0, 4, 0))
								[
									SAssignNew(SearchBox, SSearchBox)
									.OnTextChanged(InArgs._OnFiltersSearchChanged)
									.HintText_Lambda([Settings]()->FText{return Settings->bSearchInsideLogs ? LOCTEXT("DataFiltersSearchHint", "Log Data Search") : LOCTEXT("CategoryFiltersSearchHint", "Log Category Search"); })
								]
#endif
							]
						]
					]
					+ SVerticalBox::Slot()
					.FillHeight(1.0)
					[
						SNew(SInputCatcherOverlay, FLogVisualizer::Get().GetTimeSliderController().ToSharedRef())
						+ SOverlay::Slot()
						[
							MakeSectionOverlay(FLogVisualizer::Get().GetTimeSliderController().ToSharedRef(), InArgs._ViewRange, InArgs._ScrubPosition, false)
						]
						+ SOverlay::Slot()
						[
							SAssignNew(ScrollBox, SScrollBox)
							.ExternalScrollbar(ScrollBar)
							+ SScrollBox::Slot()
							[
								SAssignNew(TimelinesContainer, SVisualLoggerTimelinesContainer, SharedThis(this), FLogVisualizer::Get().GetTimeSliderController().ToSharedRef())
							]
						]
						+ SOverlay::Slot()
						[
							MakeSectionOverlay(FLogVisualizer::Get().GetTimeSliderController().ToSharedRef(), InArgs._ViewRange, InArgs._ScrubPosition, true)
						]
						+ SOverlay::Slot()
						.VAlign(VAlign_Bottom)
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.FillWidth(TAttribute<float>(this, &SVisualLoggerView::GetAnimationOutlinerFillPercentage))
							[
								// Take up space but display nothing. This is required so that all areas dependent on time align correctly
								SNullWidget::NullWidget
							]
							+ SHorizontalBox::Slot()
							.FillWidth(1.0f)
							[
								ZoomScrollBar
							]
						]
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.FillWidth(TAttribute<float>(this, &SVisualLoggerView::GetAnimationOutlinerFillPercentage))
						[
							SNew(SSpacer)
						]
						+ SHorizontalBox::Slot()
						.Padding(FMargin(0.0f, 0.0f, 0.0f, 0.0f))
						.FillWidth(1.0f)
						[
							SNew(SBorder)
							.Padding(FMargin(0.0f, 0.0f, 0.0f, 2.0f))
							.BorderImage(FLogVisualizerStyle::Get().GetBrush("ToolPanel.GroupBorder"))
							.BorderBackgroundColor(FLinearColor(.50f, .50f, .50f, 1.0f))
							[
								BottomTimeSlider
							]
						]
					]
				]
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.AutoWidth()
				[
					ScrollBar
				]
			]
		];
		
	SearchBox->SetText(FText::FromString(FVisualLoggerFilters::Get().GetSearchString()));
	FLogVisualizer::Get().GetEvents().GetAnimationOutlinerFillPercentageFunc.BindLambda(
		[this]()->float{ 
			SSplitter::FSlot const& LeftSplitterSlot = SearchSplitter->SlotAt(0);
			SSplitter::FSlot const& RightSplitterSlot = SearchSplitter->SlotAt(1);

			return LeftSplitterSlot.GetSizeValue() / RightSplitterSlot.GetSizeValue();
	});

	OnSearchSplitterResized();

	OnFiltersChangedDelegateHandle = FLogVisualizer::Get().GetEvents().OnFiltersChanged.AddSP(this, &SVisualLoggerView::OnFiltersChanged);
}

SVisualLoggerView::~SVisualLoggerView()
{
	FLogVisualizer::Get().GetEvents().GetAnimationOutlinerFillPercentageFunc.Unbind();
	FLogVisualizer::Get().GetEvents().OnFiltersChanged.Remove(OnFiltersChangedDelegateHandle);
}

void SVisualLoggerView::SetAnimationOutlinerFillPercentage(float FillPercentage) 
{ 
	AnimationOutlinerFillPercentage = FillPercentage; 
}

void SVisualLoggerView::SetSearchString(FText SearchString) 
{ 
	if (SearchBox.IsValid())
	{
		SearchBox->SetText(SearchString);
	}
}


void SVisualLoggerView::OnSearchSplitterResized()
{
	SSplitter::FSlot const& LeftSplitterSlot = SearchSplitter->SlotAt(0);
	SSplitter::FSlot const& RightSplitterSlot = SearchSplitter->SlotAt(1);

	const float NewAnimationOutlinerFillPercentage = LeftSplitterSlot.GetSizeValue() / RightSplitterSlot.GetSizeValue();
	SetAnimationOutlinerFillPercentage(NewAnimationOutlinerFillPercentage);

	FLogVisualizer::Get().SetAnimationOutlinerFillPercentage(NewAnimationOutlinerFillPercentage);
}

void SVisualLoggerView::OnSearchChanged(const FText& Filter)
{
	TimelinesContainer->OnSearchChanged(Filter);
}

TSharedRef<SWidget> SVisualLoggerView::MakeSectionOverlay(TSharedRef<FVisualLoggerTimeSliderController> TimeSliderController, const TAttribute< TRange<float> >& ViewRange, const TAttribute<float>& ScrubPosition, bool bTopOverlay)
{
	return
		SNew(SHorizontalBox)
		.Visibility(EVisibility::HitTestInvisible)
		+ SHorizontalBox::Slot()
		.FillWidth(TAttribute<float>(this, &SVisualLoggerView::GetAnimationOutlinerFillPercentage))
		[
			// Take up space but display nothing. This is required so that all areas dependent on time align correctly
			SNullWidget::NullWidget
		]
		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		[
			SNew(SVisualLoggerSectionOverlay, TimeSliderController)
			.DisplayScrubPosition(bTopOverlay)
			.DisplayTickLines(!bTopOverlay)
		];
}

void SVisualLoggerView::ResetData()
{
	TimelinesContainer->ResetData();
}

void SVisualLoggerView::OnFiltersChanged()
{
	TimelinesContainer->OnFiltersChanged();
}

void SVisualLoggerView::OnFiltersSearchChanged(const FText& Filter)
{
	TimelinesContainer->OnFiltersSearchChanged(Filter);
}

FCursorReply SVisualLoggerView::OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
{
	if (FLogVisualizer::Get().GetTimeSliderController()->IsPanning())
	{
		return FCursorReply::Cursor(EMouseCursor::GrabHand);
	}
	return FCursorReply::Cursor(EMouseCursor::Default);
}

TSharedRef<SWidget> SVisualLoggerView::MakeClassesFilterMenu()
{
	const TArray< TSharedPtr<class SLogVisualizerTimeline> >& AllTimelines = TimelinesContainer->GetAllNodes();

	FMenuBuilder MenuBuilder(true, NULL);

	TArray<FString> UniqueClasses;
	MenuBuilder.BeginSection(TEXT("Graphs"));
	for (TSharedPtr<class SLogVisualizerTimeline> CurrentTimeline : AllTimelines)
	{
		FString OwnerClassName = CurrentTimeline->GetOwnerClassName().ToString();
		if (UniqueClasses.Find(OwnerClassName) == INDEX_NONE)
		{
			FText LabelText = FText::FromString(OwnerClassName);
			MenuBuilder.AddMenuEntry(
				LabelText,
				FText::Format(LOCTEXT("FilterByClassPrefix", "Toggle {0} class"), LabelText),
				FSlateIcon(),
				FUIAction(
				FExecuteAction::CreateLambda([this, OwnerClassName]()
				{
				if (FVisualLoggerFilters::Get().MatchObjectName(OwnerClassName) && FVisualLoggerFilters::Get().GetSelectedObjects().Num() != 0)
					{
						FVisualLoggerFilters::Get().RemoveObjectFromSelection(OwnerClassName);
					}
					else
					{
						FVisualLoggerFilters::Get().SelectObject(OwnerClassName);
					}

					OnChangedClassesFilter();
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([OwnerClassName]()->bool
				{
					return FVisualLoggerFilters::Get().GetSelectedObjects().Find(OwnerClassName) != INDEX_NONE;
				}),
				FIsActionButtonVisible()),
				NAME_None,
				EUserInterfaceActionType::ToggleButton
				);
			UniqueClasses.AddUnique(OwnerClassName);
		}
	}
	//show any classes from persistent data
	for (const FString& SelectedObj : FVisualLoggerFilters::Get().GetSelectedObjects())
	{
		if (UniqueClasses.Find(SelectedObj) == INDEX_NONE)
		{
			FText LabelText = FText::FromString(SelectedObj);
			MenuBuilder.AddMenuEntry(
			LabelText,
			FText::Format(LOCTEXT("FilterByClassPrefix", "Toggle {0} class"), LabelText),
			FSlateIcon(),
			FUIAction(
			FExecuteAction::CreateLambda([this, SelectedObj]()
			{
				if (FVisualLoggerFilters::Get().MatchObjectName(SelectedObj) && FVisualLoggerFilters::Get().GetSelectedObjects().Num() != 0)
				{
					FVisualLoggerFilters::Get().RemoveObjectFromSelection(SelectedObj);
				}
				else
				{
					FVisualLoggerFilters::Get().SelectObject(SelectedObj);
				}

				OnChangedClassesFilter();
			}),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([SelectedObj]()->bool
			{
				return FVisualLoggerFilters::Get().GetSelectedObjects().Find(SelectedObj) != INDEX_NONE;
			}),
			FIsActionButtonVisible()),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
			);
			UniqueClasses.AddUnique(SelectedObj);
		}
	}
	MenuBuilder.EndSection(); 


	FDisplayMetrics DisplayMetrics;
	FSlateApplication::Get().GetCachedDisplayMetrics(DisplayMetrics);

	const FVector2D DisplaySize(
		DisplayMetrics.PrimaryDisplayWorkAreaRect.Right - DisplayMetrics.PrimaryDisplayWorkAreaRect.Left,
		DisplayMetrics.PrimaryDisplayWorkAreaRect.Bottom - DisplayMetrics.PrimaryDisplayWorkAreaRect.Top);

	return
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.MaxHeight(DisplaySize.Y * 0.9)
		[
			MenuBuilder.MakeWidget()
		];
}

void SVisualLoggerView::OnChangedClassesFilter()
{
	GetMutableDefault<ULogVisualizerSettings>()->SaveConfig();
	for (auto CurrentItem : TimelinesContainer->GetAllNodes())
	{
		CurrentItem->UpdateVisibility();
	}
}

#undef LOCTEXT_NAMESPACE
