// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/CursorReply.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Framework/Commands/UICommandList.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SScrollBox.h"
#include "SVisualLoggerBaseWidget.h"


class SVisualLoggerTimelinesContainer;

class SVisualLoggerView : public SVisualLoggerBaseWidget
{
public:
	SLATE_BEGIN_ARGS(SVisualLoggerView) 
		: _ViewRange(TRange<float>(0.0f, 5.0f))
		, _ScrubPosition(1.0f)
	{}
	/** The current view range (seconds) */
	SLATE_ATTRIBUTE(TRange<float>, ViewRange)
		/** The current scrub position in (seconds) */
		SLATE_ATTRIBUTE(float, ScrubPosition)
		SLATE_EVENT(FOnFiltersSearchChanged, OnFiltersSearchChanged)
	SLATE_END_ARGS();

	virtual ~SVisualLoggerView();
	void Construct(const FArguments& InArgs, const TSharedRef<FUICommandList>& InCommandList);
	float GetAnimationOutlinerFillPercentage() const { 
		SSplitter::FSlot const& LeftSplitterSlot = SearchSplitter->SlotAt(0);
		SSplitter::FSlot const& RightSplitterSlot = SearchSplitter->SlotAt(1);

		return LeftSplitterSlot.GetSizeValue()/ RightSplitterSlot.GetSizeValue();
	}
	void SetAnimationOutlinerFillPercentage(float FillPercentage);

	TSharedRef<SWidget> MakeSectionOverlay(TSharedRef<class FVisualLoggerTimeSliderController> TimeSliderController, const TAttribute< TRange<float> >& ViewRange, const TAttribute<float>& ScrubPosition, bool bTopOverlay);
	void SetSearchString(FText SearchString);

	void OnFiltersChanged();
	void OnSearchChanged(const FText& Filter);
	void OnFiltersSearchChanged(const FText& Filter);
	void OnSearchSplitterResized();
	void OnChangedClassesFilter();

	virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;
	TSharedRef<SWidget> MakeClassesFilterMenu();
	void ResetData();

protected:
	TSharedPtr<class SVisualLoggerTimelinesContainer> TimelinesContainer;
	TSharedPtr<class SSplitter> SearchSplitter;
	TSharedPtr<class SScrollBox> ScrollBox;
	TSharedPtr<class SSearchBox> SearchBox;
	TSharedPtr<class SComboButton> ClassesComboButton;

	FDelegateHandle OnFiltersChangedDelegateHandle;

	float AnimationOutlinerFillPercentage;
};
