// Copyright Epic Games, Inc. All Rights Reserved.

#include "VisualLoggerCameraController.h"
#include "Components/InputComponent.h"
#include "Engine/World.h"
#include "Kismet/GameplayStatics.h"
#include "GameFramework/WorldSettings.h"
#include "GameFramework/PlayerInput.h"
#include "Engine/Player.h"
#include "VisualLoggerHUD.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VisualLoggerCameraController)


//----------------------------------------------------------------------//
// AVisualLoggerCameraController
//----------------------------------------------------------------------//
TWeakObjectPtr<AVisualLoggerCameraController> AVisualLoggerCameraController::Instance;

AVisualLoggerCameraController::AVisualLoggerCameraController(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SpeedScale = 1.5f;
}

void AVisualLoggerCameraController::SetupInputComponent()
{
	Super::SetupInputComponent();

	static bool bBindingsAdded = false;
	if (!bBindingsAdded)
	{
		bBindingsAdded = true;

		UPlayerInput::AddEngineDefinedActionMapping(FInputActionKeyMapping("LogCamera_NextEntry", EKeys::LeftMouseButton, true));
		UPlayerInput::AddEngineDefinedActionMapping(FInputActionKeyMapping("LogCamera_PrevEntry", EKeys::RightMouseButton, true));
	}

	InputComponent->BindAction("LogCamera_NextEntry", IE_Pressed, this, &AVisualLoggerCameraController::ShowNextEntry);
	InputComponent->BindAction("LogCamera_PrevEntry", IE_Pressed, this, &AVisualLoggerCameraController::ShowPrevEntry);
}

void AVisualLoggerCameraController::ShowNextEntry()
{
	OnIterateLogEntries.ExecuteIfBound(1);
}

void AVisualLoggerCameraController::ShowPrevEntry()
{
	OnIterateLogEntries.ExecuteIfBound(-1);
}

void AVisualLoggerCameraController::PostInitializeComponents()
{
	Super::PostInitializeComponents();

	// if hud is existing, delete it and create new hud for debug camera
	if ( MyHUD != nullptr )
	{
		MyHUD->Destroy();
	}
	FActorSpawnParameters SpawnInfo;
	SpawnInfo.Owner = this;
	SpawnInfo.Instigator = GetInstigator();
	SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	MyHUD = GetWorld()->SpawnActor<AVisualLoggerHUD>(SpawnInfo);

	/** TO DO: somehow this two configuration settings are set to zero for this controller, have to fix it later */
	InputYawScale_DEPRECATED = 2.5f;
	InputPitchScale_DEPRECATED = -1.75f;
}


AVisualLoggerCameraController* AVisualLoggerCameraController::EnableCamera(UWorld* InWorld)
{
	APlayerController* PC = UGameplayStatics::GetPlayerController(InWorld, 0);
	if (PC == nullptr || PC->Player == nullptr || PC->IsLocalPlayerController() == false
		|| PC->GetWorld() == nullptr)
	{
		return nullptr;
	}
	
	if (Instance.IsValid() == false)
	{
		// spawn if necessary
		// and ungly @HACK to be able to spawn camera in game world rathen then
		// in editor world (if running PIE). Hate it, but it works, and 
		// this is a debugging tool		
		FActorSpawnParameters SpawnInfo;
		SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
		SpawnInfo.Owner = PC->GetWorldSettings();
		SpawnInfo.Instigator = PC->GetInstigator();
		Instance = InWorld->SpawnActor<AVisualLoggerCameraController>( SpawnInfo );
		Instance->Player = PC->Player;
	}
	
	if (Instance.IsValid())
	{
		if (Instance.Get() != PC)
		{
			// set up new controller
			Instance->OnActivate(PC);

			// then switch to it
			PC->Player->SwitchController( Instance.Get() );

			Instance->ChangeState(NAME_Default);
			Instance->ChangeState(NAME_Spectating);
		}

		return Instance.Get();
	}

	return nullptr;
}

void AVisualLoggerCameraController::DisableCamera(UWorld* InWorld)
{
	check(InWorld);
	for (FConstPlayerControllerIterator Iterator = InWorld->GetPlayerControllerIterator(); Iterator; ++Iterator)
	{
		AVisualLoggerCameraController* VLogCam = Cast<AVisualLoggerCameraController>(Iterator->Get());
		if (VLogCam && VLogCam->OriginalPlayer && VLogCam->OriginalControllerRef)
		{
			VLogCam->OriginalPlayer->SwitchController(VLogCam->OriginalControllerRef);
			VLogCam->OnDeactivate(VLogCam->OriginalControllerRef);
			InWorld->DestroyActor(VLogCam, false, false);
			break;
		}
	}
}

bool AVisualLoggerCameraController::IsEnabled(UWorld* InWorld)
{
	if (InWorld != nullptr)
	{
		for (FConstPlayerControllerIterator Iterator = InWorld->GetPlayerControllerIterator(); Iterator; ++Iterator)
		{
			AVisualLoggerCameraController* VLogCam = Cast<AVisualLoggerCameraController>(Iterator->Get());
			if (VLogCam && VLogCam->OriginalControllerRef && VLogCam->OriginalPlayer)
			{
				return true;
			}
		}
	}

	return false;
}

void AVisualLoggerCameraController::Select(FHitResult const& Hit) 
{
	Super::Select(Hit);

	if (SelectedActor != PickedActor)
	{
		PickedActor = SelectedActor;
		OnActorSelected.ExecuteIfBound(SelectedActor.Get());
	}
}

