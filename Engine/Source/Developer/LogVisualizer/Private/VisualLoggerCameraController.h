// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "Engine/DebugCameraController.h"
#include "VisualLoggerCameraController.generated.h"

UCLASS(config=Input, hidedropdown)
class AVisualLoggerCameraController : public ADebugCameraController
{
public:
	DECLARE_DELEGATE_OneParam(FActorSelectedDelegate, class AActor*);
	DECLARE_DELEGATE_OneParam(FLogEntryIterationDelegate, int32);

	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TWeakObjectPtr<class AActor> PickedActor;

	virtual void PostInitializeComponents() override;

	static AVisualLoggerCameraController* EnableCamera(UWorld* InWorld);
	static void DisableCamera(UWorld* InWorld);
	static bool IsEnabled(UWorld* InWorld);

	virtual void Select( FHitResult const& Hit ) override;

	void ShowNextEntry();
	void ShowPrevEntry();

	static TWeakObjectPtr<AVisualLoggerCameraController> Instance;

	FActorSelectedDelegate OnActorSelected;
	FLogEntryIterationDelegate OnIterateLogEntries;

protected:
	virtual void SetupInputComponent() override;
};
