// Copyright Epic Games, Inc. All Rights Reserved.

#include "VisualLoggerCanvasRenderer.h"

#if UE_DEBUG_VISUALIZER_TOOL_ENABLED
#include "EngineGlobals.h"
#include "CanvasItem.h"
#include "DebugRenderSceneProxy.h"
#include "Engine/Canvas.h"
#include "Engine/Engine.h"
#include "SceneView.h"
#include "VisualLogger/VisualLogger.h"
#include "LogVisualizerSettings.h"
#include "LogVisualizerSessionSettings.h"
#include "VisualLoggerDatabase.h"
#include "LogVisualizerPublic.h"
#include "VisualLoggerTimeSliderController.h"
#include "Debug/ReporterGraph.h"


namespace LogVisualizer
{
	FORCEINLINE bool PointInFrustum(UCanvas* Canvas, const FVector& Location)
	{
		return Canvas->SceneView->ViewFrustum.IntersectBox(Location, FVector::ZeroVector);
	}

	FORCEINLINE void DrawText(UCanvas* Canvas, UFont* Font, const FString& TextToDraw, const FVector& WorldLocation)
	{
		if (PointInFrustum(Canvas, WorldLocation))
		{
			const FVector3f ScreenLocation = UE::DebugDrawHelper::GetScaleAdjustedScreenLocation(Canvas, WorldLocation);
			Canvas->DrawText(Font, *TextToDraw, ScreenLocation.X, ScreenLocation.Y);
		}
	}

	FORCEINLINE void DrawTextCentered(UCanvas* Canvas, UFont* Font, const FString& TextToDraw, const FVector& WorldLocation)
	{
		if (PointInFrustum(Canvas, WorldLocation))
		{
			const FVector3f ScreenLocation = UE::DebugDrawHelper::GetScaleAdjustedScreenLocation(Canvas, WorldLocation);
			float TextXL = 0.f;
			float TextYL = 0.f;
			Canvas->StrLen(Font, TextToDraw, TextXL, TextYL);
			Canvas->DrawText(Font, *TextToDraw, ScreenLocation.X - TextXL / 2.0f, ScreenLocation.Y - TextYL / 2.0f);
		}
	}

	FORCEINLINE void DrawTextShadowed(UCanvas* Canvas, UFont* Font, const FString& TextToDraw, const FVector& WorldLocation)
	{
		if (PointInFrustum(Canvas, WorldLocation))
		{
			const FVector3f ScreenLocation = UE::DebugDrawHelper::GetScaleAdjustedScreenLocation(Canvas, WorldLocation);
			float TextXL = 0.f;
			float TextYL = 0.f;
			Canvas->StrLen(Font, TextToDraw, TextXL, TextYL);
			Canvas->SetDrawColor(FColor::Black);
			Canvas->DrawText(Font, *TextToDraw, 1 + ScreenLocation.X - TextXL / 2.0f, 1 + ScreenLocation.Y - TextYL / 2.0f);
			Canvas->SetDrawColor(FColor::White);
			Canvas->DrawText(Font, *TextToDraw, ScreenLocation.X - TextXL / 2.0f, ScreenLocation.Y - TextYL / 2.0f);
		}
	}
}

FVisualLoggerCanvasRenderer::FVisualLoggerCanvasRenderer() 
	: bDirtyData(true) 
{
	FLogVisualizer::Get().GetEvents().OnFiltersChanged.AddRaw(this, &FVisualLoggerCanvasRenderer::DirtyCachedData);
	FVisualLoggerDatabase::Get().GetEvents().OnRowSelectionChanged.AddRaw(this, &FVisualLoggerCanvasRenderer::ObjectSelectionChanged);
	FVisualLoggerDatabase::Get().GetEvents().OnItemSelectionChanged.AddRaw(this, &FVisualLoggerCanvasRenderer::OnItemSelectionChanged);
}

FVisualLoggerCanvasRenderer::~FVisualLoggerCanvasRenderer()
{
	FLogVisualizer::Get().GetEvents().OnFiltersChanged.RemoveAll(this);
	FVisualLoggerDatabase::Get().GetEvents().OnRowSelectionChanged.RemoveAll(this);
	FVisualLoggerDatabase::Get().GetEvents().OnItemSelectionChanged.RemoveAll(this);
}

void FVisualLoggerCanvasRenderer::ResetData()
{
	SelectedEntry = FVisualLogEntry();
	DirtyCachedData();
}

void FVisualLoggerCanvasRenderer::OnItemSelectionChanged(const FVisualLoggerDBRow& ChangedRow, int32 SelectedItemIndex)
{
	SelectedEntry = ChangedRow.GetCurrentItemIndex() != INDEX_NONE ? ChangedRow.GetCurrentItem().Entry : FVisualLogEntry();
	DirtyCachedData();
}

void FVisualLoggerCanvasRenderer::ObjectSelectionChanged(const TArray<FName>& RowNames)
{
	DirtyCachedData();
}

void FVisualLoggerCanvasRenderer::DrawOnCanvas(class UCanvas* Canvas, class APlayerController*)
{
	if (GEngine == NULL)
	{
		return;
	}

	UWorld* World = FLogVisualizer::Get().GetWorld();
	if (World == NULL)
	{
		return;
	}

	UFont* Font = GEngine->GetSmallFont();
	FCanvasTextItem TextItem(FVector2D::ZeroVector, FText::GetEmpty(), Font, FLinearColor::White);

	const FString TimeStampString = FString::Printf(TEXT("%.2f"), SelectedEntry.TimeStamp);
	LogVisualizer::DrawTextShadowed(Canvas, Font, TimeStampString, SelectedEntry.Location);

	if (bDirtyData && FLogVisualizer::Get().GetTimeSliderController().IsValid())
	{
		const FVisualLoggerTimeSliderArgs&  TimeSliderArgs = FLogVisualizer::Get().GetTimeSliderController()->GetTimeSliderArgs();
		TRange<double> LocalViewRange = TimeSliderArgs.ViewRange.Get();
		const double LocalViewRangeMin = LocalViewRange.GetLowerBoundValue();
		const double LocalViewRangeMax = LocalViewRange.GetUpperBoundValue();
		const double LocalSequenceLength = LocalViewRangeMax - LocalViewRangeMin;
		const double WindowHalfWidth = LocalSequenceLength * TimeSliderArgs.CursorSize.Get() * 0.5;
		const FVector2D TimeStampWindow(SelectedEntry.TimeStamp - WindowHalfWidth, SelectedEntry.TimeStamp + WindowHalfWidth);

		CollectedGraphs.Reset();
		const TArray<FName>& RowNames = FVisualLoggerDatabase::Get().GetSelectedRows();
		for (auto RowName : RowNames)
		{
			if (FVisualLoggerDatabase::Get().IsRowVisible(RowName) == false)
			{
				continue;
			}

			const TArray<FVisualLoggerGraph>& AllGraphs = FVisualLoggerGraphsDatabase::Get().GetGraphsByOwnerName(RowName);
			for (const FVisualLoggerGraph& CurrentGraph : AllGraphs)
			{
				const FName GraphName = CurrentGraph.GetGraphName();
				const FName OwnerName = CurrentGraph.GetOwnerName();

				if (FVisualLoggerGraphsDatabase::Get().IsGraphVisible(OwnerName, GraphName) == false)
				{
					continue;
				}

				for (auto DataIt(CurrentGraph.GetConstDataIterator()); DataIt; ++DataIt)
				{
					const FVisualLoggerGraphData& GraphData = *DataIt;
					const bool bIsGraphDataDisabled = FVisualLoggerFilters::Get().IsGraphDataDisabled(GraphName, GraphData.DataName);
					if (bIsGraphDataDisabled)
					{
						continue;
					}

					const FName FullGraphName = *FString::Printf(TEXT("%s$%s"), *RowName.ToString(), *GraphName.ToString());
					FGraphData &CollectedGraphData = CollectedGraphs.FindOrAdd(FullGraphName);
					FGraphLineData &LineData = CollectedGraphData.GraphLines.FindOrAdd(GraphData.DataName);
					LineData.DataName = GraphData.DataName;
					LineData.Samples = GraphData.Samples;
					LineData.LeftExtreme = FVector2D::ZeroVector;
					LineData.RightExtreme = FVector2D::ZeroVector;

					int32 LeftSideOutsideIndex = INDEX_NONE;
					int32 RightSideOutsideIndex = INDEX_NONE;
					for (int32 SampleIndex = 0; SampleIndex < GraphData.Samples.Num(); SampleIndex++)
					{
						const FVector2D& SampleValue = GraphData.Samples[SampleIndex];
						CollectedGraphData.Min.X = FMath::Min(CollectedGraphData.Min.X, SampleValue.X);
						CollectedGraphData.Min.Y = FMath::Min(CollectedGraphData.Min.Y, SampleValue.Y);

						CollectedGraphData.Max.X = FMath::Max(CollectedGraphData.Max.X, SampleValue.X);
						CollectedGraphData.Max.Y = FMath::Max(CollectedGraphData.Max.Y, SampleValue.Y);

						const double CurrentTimeStamp = GraphData.TimeStamps[SampleIndex];
						if (CurrentTimeStamp < TimeStampWindow.X)
						{
							LineData.LeftExtreme = SampleValue;
						}
						else if (CurrentTimeStamp > TimeStampWindow.Y)
						{
							LineData.RightExtreme = SampleValue;
							break;
						}
					}
				}
			}
		}
		bDirtyData = false;
	}

	if (ULogVisualizerSessionSettings::StaticClass()->GetDefaultObject<ULogVisualizerSessionSettings>()->bEnableGraphsVisualization)
	{
		DrawHistogramGraphs(Canvas, NULL);
	}

	const TMap<FName, FVisualLogExtensionInterface*>& Extensions = FVisualLogger::Get().GetAllExtensions();
	for (const auto& CurrentExtension : Extensions)
	{
		CurrentExtension.Value->DrawData(FVisualLoggerEditorInterface::Get(), Canvas);
	}
}

void FVisualLoggerCanvasRenderer::DrawHistogramGraphs(class UCanvas* Canvas, class APlayerController*)
{
	if (FLogVisualizer::Get().GetTimeSliderController().IsValid() == false)
	{
		return;
	}

	const float GoldenRatioConjugate = 0.618033988749895f;
	if (CollectedGraphs.Num() > 0)
	{
		const FVisualLoggerTimeSliderArgs&  TimeSliderArgs = FLogVisualizer::Get().GetTimeSliderController()->GetTimeSliderArgs();
		TRange<double> LocalViewRange = TimeSliderArgs.ViewRange.Get();
		const double LocalViewRangeMin = LocalViewRange.GetLowerBoundValue();
		const double LocalViewRangeMax = LocalViewRange.GetUpperBoundValue();
		const double LocalSequenceLength = LocalViewRangeMax - LocalViewRangeMin;
		const double WindowHalfWidth = LocalSequenceLength * TimeSliderArgs.CursorSize.Get() * 0.5f;
		const FVector2D TimeStampWindow(SelectedEntry.TimeStamp - WindowHalfWidth, SelectedEntry.TimeStamp + WindowHalfWidth);

		const FColor GraphsBackgroundColor = ULogVisualizerSettings::StaticClass()->GetDefaultObject<ULogVisualizerSettings>()->GraphsBackgroundColor;
		const int NumberOfGraphs = CollectedGraphs.Num();
		const int32 NumberOfColumns = FMath::CeilToInt(FMath::Sqrt(static_cast<float>(NumberOfGraphs)));
		int32 NumberOfRows = FMath::FloorToInt((float)NumberOfGraphs / (float)NumberOfColumns);
		if (NumberOfGraphs - NumberOfRows * NumberOfColumns > 0)
		{
			NumberOfRows += 1;
		}

		const int32 MaxNumberOfGraphs = FMath::Max(NumberOfRows, NumberOfColumns);
		const float GraphWidth = 0.8f / NumberOfColumns;
		const float GraphHeight = 0.8f / NumberOfRows;

		const float XGraphSpacing = 0.2f / (MaxNumberOfGraphs + 1);
		const float YGraphSpacing = 0.2f / (MaxNumberOfGraphs + 1);

		const float StartX = XGraphSpacing;
		float StartY = 0.5f + (0.5f * NumberOfRows - 1) * (GraphHeight + YGraphSpacing);

		float CurrentX = StartX;
		float CurrentY = StartY;
		int32 GraphIndex = 0;
		int32 CurrentColumn = 0;
		int32 CurrentRow = 0;
		bool bDrawExtremesOnGraphs = ULogVisualizerSettings::StaticClass()->GetDefaultObject<ULogVisualizerSettings>()->bDrawExtremesOnGraphs;
		const bool bConstrainGraphToLocalMinMax = ULogVisualizerSettings::StaticClass()->GetDefaultObject<ULogVisualizerSettings>()->bConstrainGraphToLocalMinMax;
		for (auto It(CollectedGraphs.CreateIterator()); It; ++It)
		{
			TWeakObjectPtr<UReporterGraph> HistogramGraph = Canvas->GetReporterGraph();
			if (!HistogramGraph.IsValid())
			{
				break;
			}
			HistogramGraph->SetNumGraphLines(It->Value.GraphLines.Num());
			int32 LineIndex = 0;
			UFont* Font = GEngine->GetSmallFont();
			int32 MaxStringSize = 0;
			float Hue = 0;

			auto& CategoriesForGraph = UsedGraphCategories.FindOrAdd(It->Key.ToString());

			It->Value.GraphLines.KeySort(FNameLexicalLess());

			if (bConstrainGraphToLocalMinMax)
			{
				It->Value.Min.Y = FLT_MAX;
				It->Value.Max.Y = -FLT_MAX;
			}

			for (auto LinesIt(It->Value.GraphLines.CreateConstIterator()); LinesIt; ++LinesIt)
			{
				const FString DataName = LinesIt->Value.DataName.ToString();
				int32 CategoryIndex = CategoriesForGraph.Find(DataName);
				if (CategoryIndex == INDEX_NONE)
				{
					CategoryIndex = CategoriesForGraph.AddUnique(DataName);
				}
				Hue = CategoryIndex * GoldenRatioConjugate;
				if (Hue > 1)
				{
					Hue -= FMath::FloorToFloat(Hue);
				}

				HistogramGraph->GetGraphLine(LineIndex)->Color = FLinearColor::MakeFromHSV8(static_cast<uint8>(Hue * 255), 200, 244);
				HistogramGraph->GetGraphLine(LineIndex)->LineName = DataName;
				HistogramGraph->GetGraphLine(LineIndex)->Data.Append(LinesIt->Value.Samples);
				HistogramGraph->GetGraphLine(LineIndex)->LeftExtreme = LinesIt->Value.LeftExtreme;
				HistogramGraph->GetGraphLine(LineIndex)->RightExtreme = LinesIt->Value.RightExtreme;

				int32 DummyY, StringSizeX;
				StringSize(Font, StringSizeX, DummyY, *LinesIt->Value.DataName.ToString());
				MaxStringSize = StringSizeX > MaxStringSize ? StringSizeX : MaxStringSize;

				++LineIndex;

				if (bConstrainGraphToLocalMinMax)
				{
					for (const FVector2D& SampleData : LinesIt->Value.Samples)
					{
						if (SampleData.X >= TimeStampWindow.X && SampleData.X <= TimeStampWindow.Y)
						{
							It->Value.Min.Y = FMath::Min(It->Value.Min.Y, SampleData.Y);
							It->Value.Max.Y = FMath::Max(It->Value.Max.Y, SampleData.Y);
						}
					}
				}
			}

			FVector2D GraphSpaceSize;
			GraphSpaceSize.Y = GraphSpaceSize.X = 0.8 / CollectedGraphs.Num();

			HistogramGraph->SetGraphScreenSize(CurrentX, CurrentX + GraphWidth, CurrentY, CurrentY + GraphHeight);
			CurrentX += GraphWidth + XGraphSpacing;
			HistogramGraph->SetAxesMinMax(FVector2D(TimeStampWindow.X, It->Value.Min.Y), FVector2D(TimeStampWindow.Y, It->Value.Max.Y));

			HistogramGraph->DrawCursorOnGraph(true);
			HistogramGraph->UseTinyFont(CollectedGraphs.Num() >= 5);
			HistogramGraph->SetCursorLocation(FloatCastChecked<float>(SelectedEntry.TimeStamp, /* Precision */ 1./16.));
			HistogramGraph->SetNumThresholds(0);
			HistogramGraph->SetStyles(EGraphAxisStyle::Grid, EGraphDataStyle::Lines);
			HistogramGraph->SetBackgroundColor(GraphsBackgroundColor);
			HistogramGraph->SetLegendPosition(/*bShowHistogramLabelsOutside false ? ELegendPosition::Outside :*/ ELegendPosition::Inside);
			HistogramGraph->OffsetDataSets(/*bOffsetDataSet*/false);
			HistogramGraph->DrawExtremesOnGraph(bDrawExtremesOnGraphs);
			HistogramGraph->bVisible = true;
			HistogramGraph->Draw(Canvas);

			++GraphIndex;

			if (++CurrentColumn >= NumberOfColumns)
			{
				CurrentColumn = 0;
				CurrentRow++;

				CurrentX = StartX;
				CurrentY -= GraphHeight + YGraphSpacing;
			}
		}
	}
}
#endif // UE_DEBUG_VISUALIZER_TOOL_ENABLED