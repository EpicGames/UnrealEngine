// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "VisualLogger/VisualLoggerDefines.h"

#if UE_DEBUG_VISUALIZER_TOOL_ENABLED
#include "VisualLogger/VisualLoggerTypes.h"

struct FVisualLoggerDBRow;

struct FVisualLoggerCanvasRenderer
{
	struct FGraphLineData
	{
		FName DataName;
		FVector2D LeftExtreme, RightExtreme;
		TArray<FVector2D> Samples;
	};

	struct FGraphData
	{
		FGraphData() : Min(FVector2D(FLT_MAX, FLT_MAX)), Max(FVector2D(-FLT_MAX, -FLT_MAX)) {}

		FVector2D Min, Max;
		TMap<FName, FGraphLineData> GraphLines;
	};

	TMap<FName, FGraphData>	CollectedGraphs;

public:
	FVisualLoggerCanvasRenderer();
	~FVisualLoggerCanvasRenderer();

	void DrawOnCanvas(class UCanvas* Canvas, class APlayerController*);
	void OnItemSelectionChanged(const FVisualLoggerDBRow& ChangedRow, int32 SelectedItemIndex);
	void ObjectSelectionChanged(const TArray<FName>& RowNames);
	void DirtyCachedData() { bDirtyData = true; }
	void ResetData();

protected:
	void DrawHistogramGraphs(class UCanvas* Canvas, class APlayerController* );

private:
	bool bDirtyData;
	FVisualLogEntry SelectedEntry;
	TMap<FString, TArray<FString> > UsedGraphCategories;
};
#endif // UE_DEBUG_VISUALIZER_TOOL_ENABLED