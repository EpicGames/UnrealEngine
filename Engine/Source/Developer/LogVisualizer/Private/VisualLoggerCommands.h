// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/Commands.h"

#define LOCTEXT_NAMESPACE "FVisualLoggerCommands"

class FVisualLoggerCommands : public TCommands<FVisualLoggerCommands>
{
public:

	/** Default constructor. */
	FVisualLoggerCommands()
		: TCommands<FVisualLoggerCommands>("VisualLogger", NSLOCTEXT("Contexts", "VisualLogger", "Visual Logger"), NAME_None, "LogVisualizerStyle")
	{ }

public:

	// TCommands interface

	virtual void RegisterCommands() override
	{
		UI_COMMAND(StartRecording, "Start", "Start the debugger", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(StopRecording, "Stop", "Step over the current message", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(Pause, "Pause", "Stop the debugger", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(Resume, "Resume", "Resume", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(FreeCamera, "FreeCamera", "Enable free camera", EUserInterfaceActionType::ToggleButton, FInputChord());
		UI_COMMAND(LoadFromVLog, "Load", "Load external vlogs", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(SaveToVLog, "Save", "Save selected data to vlog file", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(SaveAllToVLog, "SaveAll", "Save all to vlog file", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(ToggleGraphs, "ToggleGraphs", "Toggle graphs visualization on/off", EUserInterfaceActionType::ToggleButton, FInputChord());
		UI_COMMAND(ResetData, "Clear", "Clear all data", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(Refresh, "Refresh", "Refresh UI from memory buffer", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(AutoScroll, "AutoScroll", "Maintains the cursor on the last recorded event", EUserInterfaceActionType::ToggleButton, FInputChord());
		UI_COMMAND(HideEmpty, "HideEmpty", "Hide logs without text information", EUserInterfaceActionType::ToggleButton, FInputChord());
	}

public:

	TSharedPtr<FUICommandInfo> StartRecording;
	TSharedPtr<FUICommandInfo> StopRecording;
	TSharedPtr<FUICommandInfo> Pause;
	TSharedPtr<FUICommandInfo> Resume;
	TSharedPtr<FUICommandInfo> FreeCamera;
	TSharedPtr<FUICommandInfo> LoadFromVLog;
	TSharedPtr<FUICommandInfo> SaveToVLog;
	TSharedPtr<FUICommandInfo> SaveAllToVLog;
	TSharedPtr<FUICommandInfo> ToggleGraphs;
	TSharedPtr<FUICommandInfo> ResetData;
	TSharedPtr<FUICommandInfo> Refresh;
	TSharedPtr<FUICommandInfo> AutoScroll;
	TSharedPtr<FUICommandInfo> HideEmpty;
};


#undef LOCTEXT_NAMESPACE
