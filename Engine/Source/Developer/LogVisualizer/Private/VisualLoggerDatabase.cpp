// Copyright Epic Games, Inc. All Rights Reserved.

#include "VisualLoggerDatabase.h"

#if UE_DEBUG_VISUALIZER_TOOL_ENABLED
#include "Engine/EngineTypes.h"
#include "Engine/World.h"
#include "EngineUtils.h"
#include "LogVisualizerSettings.h"
#include "LogVisualizerPublic.h"
#include "VisualLoggerRenderingActor.h"

TSharedPtr< struct FVisualLoggerDatabase > FVisualLoggerDatabase::StaticInstance = nullptr;
TSharedPtr< struct FVisualLoggerGraphsDatabase > FVisualLoggerGraphsDatabase::StaticInstance = nullptr;

void FVisualLoggerDBRow::AddItem(const FVisualLogDevice::FVisualLogEntryItem& NewItem) 
{ 
	const int32 ItemIndex = Items.Add(NewItem);
	const int32 HiddenIndex = HiddenItems.Add(false);
	checkf(ItemIndex == HiddenIndex, TEXT("Items and HiddenItems should have matching indices."));

	DBEvents.OnNewItem.Broadcast(*this, ItemIndex);
}

void FVisualLoggerDBRow::MoveTo(int32 Index) 
{ 
	const int32 OldItemIndex = CurrentItemIndex;
	CurrentItemIndex = Items.IsValidIndex(Index) ? Index : INDEX_NONE; 
	if (OldItemIndex != CurrentItemIndex)
	{
		DBEvents.OnItemSelectionChanged.Broadcast(*this, CurrentItemIndex);
	}
}

const FVisualLogDevice::FVisualLogEntryItem& FVisualLoggerDBRow::GetCurrentItem() const 
{ 
	check(Items.IsValidIndex(CurrentItemIndex)); 
	return Items[CurrentItemIndex]; 
}

void FVisualLoggerDBRow::SetItemVisibility(int32 ItemIndex, bool IsVisible)
{ 
	const bool IsHidden = !IsVisible;
	if (HiddenItems[ItemIndex] != IsHidden)
	{
		HiddenItems[ItemIndex] = IsHidden;
		NumHiddenItems += IsHidden ? 1 : -1;
		check(NumHiddenItems >= 0);
	}
}

int32 FVisualLoggerDBRow::GetClosestItem(double Time) const
{
	int32 BestItemIndex = INDEX_NONE;
	double BestDistance = MAX_dbl;
	for (int32 Index = 0; Index < Items.Num(); Index++)
	{
		auto& CurrentEntryItem = Items[Index];

		if (IsItemVisible(Index) == false)
		{
			continue;
		}
		TArray<FVisualLoggerCategoryVerbosityPair> OutCategories;
		const double CurrentDist = FMath::Abs(Time - CurrentEntryItem.Entry.TimeStamp);
		if (CurrentDist < BestDistance)
		{
			BestDistance = CurrentDist;
			BestItemIndex = Index;
		}
	}

	const double CurrentDist = Items.IsValidIndex(CurrentItemIndex) && IsItemVisible(CurrentItemIndex) ? FMath::Abs(Time - Items[CurrentItemIndex].Entry.TimeStamp) : MAX_dbl;
	if (BestItemIndex != INDEX_NONE && CurrentDist > BestDistance)
	{
		return BestItemIndex;
	}

	return CurrentItemIndex;
}

int32 FVisualLoggerDBRow::GetClosestItem(double Time, double ScrubTime) const
{
	int32 BestItemIndex = INDEX_NONE;
	double BestDistance = MAX_dbl;
	for (int32 Index = 0; Index < Items.Num(); Index++)
	{
		auto& CurrentEntryItem = Items[Index];

		if (CurrentEntryItem.Entry.TimeStamp > ScrubTime)
		{
			break;
		}

		if (IsItemVisible(Index) == false)
		{
			continue;
		}
		TArray<FVisualLoggerCategoryVerbosityPair> OutCategories;
		const double CurrentDist = FMath::Abs(CurrentEntryItem.Entry.TimeStamp - Time);
		if (CurrentDist < BestDistance && CurrentEntryItem.Entry.TimeStamp <= Time)
		{
			BestDistance = CurrentDist;
			BestItemIndex = Index;
		}
	}

	if (BestItemIndex != INDEX_NONE)
	{
		return BestItemIndex;
	}

	return CurrentItemIndex;
}

FVisualLoggerDatabase& FVisualLoggerDatabase::Get()
{
	return *StaticInstance;
}

void FVisualLoggerDatabase::Initialize()
{
	StaticInstance = MakeShareable(new FVisualLoggerDatabase);
	FVisualLoggerGraphsDatabase::Initialize();
}

void FVisualLoggerDatabase::Shutdown()
{
	FVisualLoggerGraphsDatabase::Shutdown();
	StaticInstance.Reset();
}

void FVisualLoggerDatabase::Reset()
{
	Rows.Reset();
	RowNameToIndex.Reset();
	SelectedRows.Reset();
	HiddenRows.Reset();
	FVisualLoggerGraphsDatabase::Get().Reset();
}

void FVisualLoggerDatabase::AddItem(const FVisualLogDevice::FVisualLogEntryItem& NewItem)
{
	const bool bCreateNew = RowNameToIndex.Contains(NewItem.OwnerName) == false;
	int32 RowIndex = INDEX_NONE;
	FVisualLoggerDBRow* CurrentRow = nullptr;

	if (bCreateNew)
	{
		RowIndex = Rows.Add(FVisualLoggerDBRow(DBEvents, NewItem.OwnerName, NewItem.OwnerDisplayName, NewItem.OwnerClassName));
		RowNameToIndex.Add(NewItem.OwnerName, RowIndex);
		CurrentRow = &Rows[RowIndex];
		DBEvents.OnNewRow.Broadcast(*CurrentRow);
	}
	else
	{
		RowIndex = RowNameToIndex[NewItem.OwnerName];
		if (ensure(Rows.IsValidIndex(RowIndex)))
		{
			CurrentRow = &Rows[RowIndex];
		}
	}
	
	if (CurrentRow)
	{
		CurrentRow->AddItem(NewItem);
		FVisualLoggerGraphsDatabase::Get().AddItem(NewItem);
	}
}

bool FVisualLoggerDatabase::ContainsRowByName(FName InName)
{
	return RowNameToIndex.Contains(InName);
}

FVisualLoggerDBRow& FVisualLoggerDatabase::GetRowByName(FName InName)
{
	const int32* Idx = RowNameToIndex.Find(InName);
	check(Idx);

	return Rows[*Idx];
}

void FVisualLoggerDatabase::SelectRow(FName InName, bool bDeselectOtherNodes)
{
	const bool bAlreadySelected = SelectedRows.Find(InName) != INDEX_NONE;
	if (bAlreadySelected && (!bDeselectOtherNodes || SelectedRows.Num() == 1))
	{
		return;
	}

	if (bDeselectOtherNodes)
	{
		for (auto CurrentName : SelectedRows)
		{
			if (CurrentName != InName)
			{
				FVisualLoggerDBRow& DBRow = GetRowByName(CurrentName);
				DBRow.MoveTo(INDEX_NONE);
			}
		}
		SelectedRows.Reset();
		if (bAlreadySelected)
		{
			SelectedRows.AddUnique(InName);
			DBEvents.OnRowSelectionChanged.Broadcast(SelectedRows);
			return;
		}
	}

	if (!bAlreadySelected)
	{
		SelectedRows.AddUnique(InName);
		DBEvents.OnRowSelectionChanged.Broadcast(SelectedRows);
	}
}

void FVisualLoggerDatabase::DeselectRow(FName InName)
{
	const bool bSelected = SelectedRows.Find(InName) != INDEX_NONE;
	if (bSelected)
	{
		FVisualLoggerDBRow& DBRow = GetRowByName(InName);
		DBRow.MoveTo(INDEX_NONE);

		SelectedRows.RemoveSingleSwap(InName, EAllowShrinking::No);
		DBEvents.OnRowSelectionChanged.Broadcast(SelectedRows);
	}
}

const TArray<FName>& FVisualLoggerDatabase::GetSelectedRows() const
{
	return SelectedRows;
}

void FVisualLoggerDatabase::SetRowVisibility(FName RowName, bool SetAsVisible)
{
	const bool IsVisible = HiddenRows.Find(RowName) == INDEX_NONE;
	if (IsVisible != SetAsVisible)
	{
		if (SetAsVisible)
		{
			HiddenRows.RemoveSingleSwap(RowName);
		}
		else
		{
			HiddenRows.AddUnique(RowName);
		}

		DBEvents.OnRowChangedVisibility.Broadcast(RowName);
	}
}

void FVisualLoggerDatabase::RemoveRow(FName RowName)
{
	if (SelectedRows.Find(RowName) != INDEX_NONE)
	{
		SelectedRows.RemoveSwap(RowName);
	}
	if (HiddenRows.Find(RowName) != INDEX_NONE)
	{
		HiddenRows.RemoveSwap(RowName);
	}
	if (RowNameToIndex.Contains(RowName))
	{
		const int32 RemovedIndex = RowNameToIndex.FindAndRemoveChecked(RowName);
		Rows.RemoveAtSwap(RemovedIndex, EAllowShrinking::No);
		if (Rows.IsValidIndex(RemovedIndex))
		{
			RowNameToIndex[Rows[RemovedIndex].GetOwnerName()] = RemovedIndex;
		}
	}
}

/************************************************************************/
/*                                                                      */
/************************************************************************/

bool FVisualLoggerGraph::IsDataVisible(FName DataName) const
{
	return HiddenGraphs.Contains(DataName) == false;
}

void FVisualLoggerGraph::SetDataVisibility(FName DataName, bool IsVisible)
{
	if (IsVisible)
	{
		HiddenGraphs.Remove(DataName);
	}
	else
	{
		HiddenGraphs.AddUnique(DataName);
	}
}

FVisualLoggerGraphData& FVisualLoggerGraph::FindOrAddDataByName(FName DataName)
{
	if (DataNameToIndex.Contains(DataName))
	{
		return DataGraphs[DataNameToIndex[DataName]];
	}

	const int32 Index = DataGraphs.Add(FVisualLoggerGraphData(DataName));
	DataNameToIndex.Add(DataName, Index);
	FVisualLoggerDatabase::Get().GetEvents().OnGraphDataNameAddedEvent.Broadcast(OwnerName, GraphName, DataName);

	return DataGraphs[Index];
}

/************************************************************************/
/*                                                                      */
/************************************************************************/

FVisualLoggerGraphsDatabase& FVisualLoggerGraphsDatabase::Get()
{
	return *StaticInstance;
}

void FVisualLoggerGraphsDatabase::Initialize()
{
	StaticInstance = MakeShareable(new FVisualLoggerGraphsDatabase);
}

void FVisualLoggerGraphsDatabase::Shutdown()
{
	StaticInstance.Reset();
}

void FVisualLoggerGraphsDatabase::Reset()
{
	OwnerNameToGraphs.Reset();
	HiddenGraphs.Reset();
}

void FVisualLoggerGraphsDatabase::AddItem(const FVisualLogDevice::FVisualLogEntryItem& NewItem)
{
	FVisualLoggerGraphHelper* GraphHelperPtr = OwnerNameToGraphs.Find(NewItem.OwnerName);
	if (!GraphHelperPtr)
	{
		GraphHelperPtr = &OwnerNameToGraphs.Add(NewItem.OwnerName);
	}
	FVisualLoggerGraphHelper& GraphHelper = *GraphHelperPtr;

	for (const FVisualLogHistogramSample& HistogramSample : NewItem.Entry.HistogramSamples)
	{
		int32 GraphIndex = INDEX_NONE;
		if (GraphHelper.GraphNameToIndex.Contains(HistogramSample.GraphName) == false)
		{
			FVisualLoggerGraph Graph(NewItem.OwnerName);
			Graph.SetGraphName(HistogramSample.GraphName);
			GraphIndex = GraphHelper.AllGraphs.Add(Graph);
			GraphHelper.GraphNameToIndex.Add(HistogramSample.GraphName, GraphIndex);

			FVisualLoggerDatabase::Get().GetEvents().OnGraphAddedEvent.Broadcast(NewItem.OwnerName, HistogramSample.GraphName);
		}
		else
		{
			GraphIndex = GraphHelper.GraphNameToIndex[HistogramSample.GraphName];
		}

		check(GraphIndex != INDEX_NONE);
		FVisualLoggerGraphData& GraphData = GraphHelper.AllGraphs[GraphIndex].FindOrAddDataByName(HistogramSample.DataName);
		GraphData.Samples.Add(HistogramSample.SampleValue);
		GraphData.TimeStamps.Add(NewItem.Entry.TimeStamp);
	}
}

bool FVisualLoggerGraphsDatabase::IsGraphVisible(FName OwnerName, FName GraphName)
{
	return HiddenGraphs.Find(*(OwnerName.ToString() + TEXT("$") + GraphName.ToString())) == INDEX_NONE;
}

void FVisualLoggerGraphsDatabase::SetGraphVisibility(FName OwnerName, FName GraphName, bool SetAsVisible)
{
	const FName FullName = *(OwnerName.ToString() + TEXT("$") + GraphName.ToString());
	const bool IsVisible = HiddenGraphs.Find(FullName) == INDEX_NONE;
	if (IsVisible != SetAsVisible)
	{
		if (SetAsVisible)
		{
			HiddenGraphs.RemoveSingleSwap(FullName);
		}
		else
		{
			HiddenGraphs.Add(FullName);
		}

		FVisualLoggerDatabase::Get().GetEvents().OnGraphChangedVisibilityEvent.Broadcast(GraphName);
	}
}

bool FVisualLoggerGraphsDatabase::ContainsGraphByName(FName OwnerName, FName GraphName)
{
	const FVisualLoggerGraphHelper& GraphHelper = OwnerNameToGraphs.FindOrAdd(OwnerName);
	return GraphHelper.GraphNameToIndex.Contains(GraphName);
}

FVisualLoggerGraph& FVisualLoggerGraphsDatabase::GetGraphByName(FName OwnerName, FName GraphName)
{
	FVisualLoggerGraphHelper& GraphHelper = OwnerNameToGraphs.FindOrAdd(OwnerName);
	const bool bContainsRow = GraphHelper.GraphNameToIndex.Contains(GraphName);
	check(bContainsRow);

	const int32 GraphIndex = GraphHelper.GraphNameToIndex[GraphName];
	return GraphHelper.AllGraphs[GraphIndex];
}

const TArray<FVisualLoggerGraph>& FVisualLoggerGraphsDatabase::GetGraphsByOwnerName(FName OwnerName)
{
	const FVisualLoggerGraphHelper& GraphHelper = OwnerNameToGraphs.FindOrAdd(OwnerName);
	return GraphHelper.AllGraphs;
}

/************************************************************************/
/* FVisualLoggerEditorInterface                                         */
/************************************************************************/
const FName& FVisualLoggerEditorInterface::GetRowClassName(FName RowName) const 
{ 
	return FVisualLoggerDatabase::Get().GetRowByName(RowName).GetOwnerClassName(); 
}

int32 FVisualLoggerEditorInterface::GetSelectedItemIndex(FName RowName) const 
{ 
	return FVisualLoggerDatabase::Get().GetRowByName(RowName).GetCurrentItemIndex(); 
}

const TArray<FVisualLogDevice::FVisualLogEntryItem>& FVisualLoggerEditorInterface::GetRowItems(FName RowName) 
{ 
	return FVisualLoggerDatabase::Get().GetRowByName(RowName).GetItems(); 
}

const FVisualLogDevice::FVisualLogEntryItem& FVisualLoggerEditorInterface::GetSelectedItem(FName RowName) const 
{ 
	return FVisualLoggerDatabase::Get().GetRowByName(RowName).GetCurrentItem(); 
}

const TArray<FName>& FVisualLoggerEditorInterface::GetSelectedRows() const 
{ 
	return FVisualLoggerDatabase::Get().GetSelectedRows(); 
}

bool FVisualLoggerEditorInterface::IsRowVisible(FName RowName) const
{ 
	return FVisualLoggerDatabase::Get().IsRowVisible(RowName); 
}

UWorld* FVisualLoggerEditorInterface::GetWorld() const
{ 
	return FLogVisualizer::Get().GetWorld(); 
}

bool FVisualLoggerEditorInterface::IsItemVisible(FName RowName, int32 ItemIndex) const
{
	return FVisualLoggerDatabase::Get().GetRowByName(RowName).IsItemVisible(ItemIndex);
}

AActor* FVisualLoggerEditorInterface::GetHelperActor(UWorld* InWorld) const
{
	UWorld* World = InWorld ? InWorld : GetWorld();
	if (World == nullptr)
	{
		return nullptr;
	}

	if (TActorIterator<AVisualLoggerRenderingActor> It(World); It)
	{
		return *It;
	}

	FActorSpawnParameters SpawnInfo;
	SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;

	// The helper actor is created on demand and only once per world so we can allow it to spawn during construction script.
	SpawnInfo.bAllowDuringConstructionScript = true;

#if WITH_EDITOR
	// Nothing to set on this actor, se we just hide it from the outliner : 
	SpawnInfo.bHideFromSceneOutliner = true;
#endif //WITH_EDITOR

	return World->SpawnActor<AVisualLoggerRenderingActor>(SpawnInfo);
}

bool FVisualLoggerEditorInterface::MatchCategoryFilters(const FString& String, ELogVerbosity::Type Verbosity)
{
	return FVisualLoggerFilters::Get().ShouldDisplayCategoryByString(String, Verbosity);
}
#endif // UE_DEBUG_VISUALIZER_TOOL_ENABLED