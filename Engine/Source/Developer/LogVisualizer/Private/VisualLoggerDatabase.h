// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "VisualLogger/VisualLoggerDefines.h"

#if UE_DEBUG_VISUALIZER_TOOL_ENABLED
#include "VisualLogger/VisualLoggerTypes.h"

class AActor;
struct FVisualLoggerDBRow;

struct FVisualLoggerDBEvents
{
	DECLARE_MULTICAST_DELEGATE_TwoParams(FItemSelectionChangedEvent, const FVisualLoggerDBRow&, int32);
	DECLARE_MULTICAST_DELEGATE_TwoParams(FNewItemEvent, const FVisualLoggerDBRow&, int32);
	DECLARE_MULTICAST_DELEGATE_OneParam(FNewRowEvent, const FVisualLoggerDBRow&);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnRowSelectionChangedEvent, const TArray<FName>&);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnRowChangedVisibilityEvent, const FName&);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnRowRemovedEvent, const FName&);

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnGraphChangedVisibilityEvent, const FName&);
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnGraphAddedEvent, const FName&, const FName&);
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnGraphDataNameAddedEvent, const FName&, const FName&, const FName&);

	FNewItemEvent OnNewItem;
	FItemSelectionChangedEvent OnItemSelectionChanged;
	FNewRowEvent OnNewRow;
	FOnRowSelectionChangedEvent OnRowSelectionChanged;
	FOnRowChangedVisibilityEvent OnRowChangedVisibility;
	FOnRowRemovedEvent OnRowRemoved;

	FOnGraphChangedVisibilityEvent OnGraphChangedVisibilityEvent;
	FOnGraphAddedEvent OnGraphAddedEvent;
	FOnGraphDataNameAddedEvent OnGraphDataNameAddedEvent;
};

struct FVisualLoggerDBRow
{
public:
	FVisualLoggerDBRow(FVisualLoggerDBEvents& InEvents, const FName& InOwnerName, const FName& InOwnerDisplayName, const FName& InOwnerClassName) 
		: DBEvents(InEvents)
		, OwnerName(InOwnerName)
		, OwnerDisplayName(InOwnerDisplayName)
		, OwnerClassName(InOwnerClassName)
	{}

	const FName& GetOwnerName() const { return OwnerName; }
	const FName& GetOwnerDisplayName() const { return OwnerDisplayName; }
	const FName& GetOwnerClassName() const { return OwnerClassName; }

	void AddItem(const FVisualLogDevice::FVisualLogEntryItem& NewItem);
	const TArray<FVisualLogDevice::FVisualLogEntryItem>& GetItems() const { return Items; }

	void MoveTo(int32 Index);
	const FVisualLogDevice::FVisualLogEntryItem& GetCurrentItem() const;
	int32 GetCurrentItemIndex() const { return CurrentItemIndex; }

	bool IsRowVisible() const
	{
		return !bHiddenBySearchFilter
			&& GetNumberOfHiddenItems() != GetItems().Num();
	}
	void SetItemVisibility(int32 ItemIndex, bool IsVisible);
	bool IsItemVisible(int32 ItemIndex) const { return HiddenItems[ItemIndex] == false; }
	int32 GetNumberOfHiddenItems() const { return NumHiddenItems; }
	void SetHiddenBySearchFilter(const bool bInHiddenBySearchFilter)
	{
		bHiddenBySearchFilter = bInHiddenBySearchFilter;
	}


	int32 GetClosestItem(double Time) const;
	int32 GetClosestItem(double Time, double ScrubTime) const;

protected:
	FVisualLoggerDBEvents& DBEvents;
	FName OwnerName;
	FName OwnerDisplayName;
	FName OwnerClassName;
	int32 CurrentItemIndex = INDEX_NONE;

	TArray<FVisualLogDevice::FVisualLogEntryItem> Items;
	TBitArray<> HiddenItems;
	int32 NumHiddenItems = 0;
	bool bHiddenBySearchFilter = false;
};

struct FVisualLoggerDatabase
{
	typedef TArray<FVisualLoggerDBRow>::TConstIterator FConstRowIterator;
	typedef TArray<FVisualLoggerDBRow>::TIterator FRowIterator;

	static FVisualLoggerDatabase& Get();
	static void Initialize();
	static void Shutdown();

	void Reset();
	FVisualLoggerDBEvents& GetEvents() { return DBEvents; }

	int32 NumberOfRows() { return Rows.Num(); }
	void AddItem(const FVisualLogDevice::FVisualLogEntryItem& NewItem);
	FConstRowIterator GetConstRowIterator() const { return Rows.CreateConstIterator(); }
	FRowIterator GetRowIterator() { return Rows.CreateIterator(); }
	
	bool ContainsRowByName(FName InName);
	FVisualLoggerDBRow& GetRowByName(FName InName);

	void SelectRow(FName InName, bool bDeselectOtherNodes = false);
	void DeselectRow(FName InName);
	bool IsRowSelected(FName InName) const { return SelectedRows.Find(InName) != INDEX_NONE; }
	const TArray<FName>& GetSelectedRows() const;

	bool IsRowVisible(FName RowName) const { return HiddenRows.Find(RowName) == INDEX_NONE; }
	void SetRowVisibility(FName RowName, bool IsVisible);

	void RemoveRow(FName RowName);

protected:
	TArray<FVisualLoggerDBRow>	Rows;
	TMap<FName, int32>	RowNameToIndex;
	TArray<FName> SelectedRows;
	TArray<FName> HiddenRows;

	FVisualLoggerDBEvents DBEvents;

private:
	static TSharedPtr< struct FVisualLoggerDatabase > StaticInstance;

};

struct FVisualLoggerGraphData
{
	FName DataName;
	TArray<FVector2D> Samples;
	TArray<double> TimeStamps;

	FVisualLoggerGraphData(FName InDataName) : DataName(InDataName) {}
};

struct FVisualLoggerGraph
{
	typedef TArray<FVisualLoggerGraphData>::TConstIterator FConstDataIterator;
	typedef TArray<FVisualLoggerGraphData>::TIterator FDataIterator;

	FVisualLoggerGraph(FName InOwnerName) : OwnerName(InOwnerName) {}

	FName GetOwnerName() const { return OwnerName; }
	FName GetGraphName() const { return GraphName; }
	void SetGraphName(FName InGraphName) { GraphName = InGraphName; }

	bool IsDataVisible(FName DataName) const;
	void SetDataVisibility(FName DataName, bool IsVisible);

	bool ContainsDataByName(FName DataName) const { return DataNameToIndex.Contains(DataName); };
	FVisualLoggerGraphData& FindOrAddDataByName(FName DataName);

	FConstDataIterator GetConstDataIterator() const { return DataGraphs.CreateConstIterator(); }
	FDataIterator GetDataIterator() { return DataGraphs.CreateIterator(); }

protected:
	FName OwnerName;
	FName GraphName;
	TArray<FVisualLoggerGraphData> DataGraphs;
	TMap<FName, int32>	DataNameToIndex;
	TArray<FName> HiddenGraphs;
};

struct FVisualLoggerGraphsDatabase //histogram graphs database as separate structure to optimize access and category filters
{
protected:
	struct FVisualLoggerGraphHelper
	{
		TArray<FVisualLoggerGraph> AllGraphs;
		TMap<FName, int32> GraphNameToIndex;
	};

public:
	typedef TArray<FVisualLoggerGraph>::TConstIterator FConstGraphIterator;
	typedef TArray<FVisualLoggerGraph>::TIterator FGraphIterator;
	
	typedef TMap<FName, FVisualLoggerGraphHelper>::TConstIterator FConstOwnersIterator;
	typedef TMap<FName, FVisualLoggerGraphHelper>::TIterator FOwnersIterator;

	static FVisualLoggerGraphsDatabase& Get();
	static void Initialize();
	static void Shutdown();
	void Reset();

	void AddItem(const FVisualLogDevice::FVisualLogEntryItem& NewItem);

	bool IsGraphVisible(FName OwnerName, FName GraphName);
	void SetGraphVisibility(FName OwnerName, FName GraphName, bool IsVisible);

	bool ContainsGraphByName(FName OwnerName, FName GraphName);
	FVisualLoggerGraph& GetGraphByName(FName OwnerName, FName GraphName);
	bool ContainsHistogramGraphs() const { return OwnerNameToGraphs.Num() > 0; }

	const TArray<FVisualLoggerGraph>& GetGraphsByOwnerName(FName OwnerName);

	FConstGraphIterator GetConstGraphsIterator(FName OwnerName) { return OwnerNameToGraphs.FindOrAdd(OwnerName).AllGraphs.CreateConstIterator(); }
	FGraphIterator GetGraphsIterator(FName OwnerName) { return OwnerNameToGraphs.FindOrAdd(OwnerName).AllGraphs.CreateIterator(); }

	FConstOwnersIterator GetConstOwnersIterator() { return OwnerNameToGraphs.CreateConstIterator(); }
	FOwnersIterator GetOwnersIterator() { return OwnerNameToGraphs.CreateIterator(); }

protected:
	TMap<FName, FVisualLoggerGraphHelper> OwnerNameToGraphs;
	TArray<FName> HiddenGraphs; //encoded as "OwnerName$GraphName" for simplicity.

private:
	static TSharedPtr< struct FVisualLoggerGraphsDatabase > StaticInstance;
};

struct FVisualLoggerEditorInterface : public IVisualLoggerEditorInterface
{
	static IVisualLoggerEditorInterface* Get() { static FVisualLoggerEditorInterface EditorInterface; return &EditorInterface; }

	const FName& GetRowClassName(FName RowName) const override;
	int32 GetSelectedItemIndex(FName RowName) const override;
	const TArray<FVisualLogDevice::FVisualLogEntryItem>& GetRowItems(FName RowName) override;
	const FVisualLogDevice::FVisualLogEntryItem& GetSelectedItem(FName RowName) const override;

	const TArray<FName>& GetSelectedRows() const override;
	bool IsRowVisible(FName RowName) const override;
	bool IsItemVisible(FName RowName, int32 ItemIndex) const override;
	UWorld* GetWorld() const override;
	AActor* GetHelperActor(UWorld* InWorld = nullptr) const override;

	bool MatchCategoryFilters(const FString& String, ELogVerbosity::Type Verbosity = ELogVerbosity::All) override;
};
#endif // UE_DEBUG_VISUALIZER_TOOL_ENABLED
