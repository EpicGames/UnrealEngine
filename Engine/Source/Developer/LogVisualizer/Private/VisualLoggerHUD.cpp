// Copyright Epic Games, Inc. All Rights Reserved.

#include "VisualLoggerHUD.h"
#include "CollisionQueryParams.h"
#include "CanvasItem.h"
#include "Engine/Canvas.h"
#include "EngineGlobals.h"
#include "Engine/Engine.h"
#include "DrawDebugHelpers.h"
#include "VisualLoggerCameraController.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VisualLoggerHUD)

#define LOCTEXT_NAMESPACE "AVisualLoggerHUD"

//----------------------------------------------------------------------//
// AVisualLoggerHUD
//----------------------------------------------------------------------//
AVisualLoggerHUD::AVisualLoggerHUD(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetHidden(false);

	TextRenderInfo.bEnableShadow = true;
}

bool AVisualLoggerHUD::DisplayMaterials( float X, float& Y, float DY, UMeshComponent* MeshComp )
{
	return false;
}

void AVisualLoggerHUD::PostRender()
{
	static const FColor TextColor(200, 200, 128, 255);

	Super::Super::PostRender();

	if (bShowHUD)
	{
		AVisualLoggerCameraController* DebugCamController = Cast<AVisualLoggerCameraController>(PlayerOwner);
		if( DebugCamController != NULL )
		{
			FCanvasTextItem TextItem( FVector2D::ZeroVector, FText::GetEmpty(), GEngine->GetSmallFont(), TextColor);
			TextItem.FontRenderInfo = TextRenderInfo;
			float X = Canvas->SizeX * 0.025f+1;
			float Y = Canvas->SizeX * 0.025f+1;

			FVector const CamLoc = DebugCamController->PlayerCameraManager->GetCameraLocation();
			FRotator const CamRot = DebugCamController->PlayerCameraManager->GetCameraRotation();

			FCollisionQueryParams TraceParams(NAME_None, FCollisionQueryParams::GetUnknownStatId(), true, this);
			FHitResult Hit;
			bool bHit = GetWorld()->LineTraceSingleByChannel(Hit, CamLoc, CamRot.Vector() * 100000.f + CamLoc, ECC_Pawn, TraceParams);
			if( bHit )
			{
				TextItem.Text = FText::FromString(FString::Printf(TEXT("Under cursor: '%s'"), *Hit.HitObjectHandle.GetName()));
				Canvas->DrawItem( TextItem, X, Y );
				
				DrawDebugLine( GetWorld(), Hit.Location, Hit.Location+Hit.Normal*30.f, FColor::White );
			}
			else
			{
				TextItem.Text = LOCTEXT("NotActorUnderCursor", "Not actor under cursor" );
			}
			Canvas->DrawItem( TextItem, X, Y );
			Y += static_cast<float>(TextItem.DrawnSize.Y);

			if (DebugCamController->PickedActor.IsValid())
			{
				TextItem.Text = FText::FromString(FString::Printf(TEXT("Selected: '%s'"), *DebugCamController->PickedActor->GetName()));
				Canvas->DrawItem( TextItem, X, Y );				
			}
		}
	}
}
#undef LOCTEXT_NAMESPACE

