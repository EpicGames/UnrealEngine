// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "Engine/DebugCameraHUD.h"
#include "VisualLoggerHUD.generated.h"

class UMeshComponent;

UCLASS(hidedropdown)
class AVisualLoggerHUD : public ADebugCameraHUD
{
	GENERATED_UCLASS_BODY()

	FFontRenderInfo TextRenderInfo;

	virtual bool DisplayMaterials( float X, float& Y, float DY, UMeshComponent* MeshComp ) override;

	//~ Begin AActor Interface
	virtual void PostRender() override;
	//~ End AActor Interface
};



