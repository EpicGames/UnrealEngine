// Copyright Epic Games, Inc. All Rights Reserved.

#include "VisualLoggerRenderingActor.h"
#include "AI/NavigationSystemBase.h"
#include "VisualLogger/VisualLogger.h"
#include "VisualLoggerDatabase.h"
#include "LogVisualizerSettings.h"
#include "LogVisualizerPublic.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VisualLoggerRenderingActor)

AVisualLoggerRenderingActor::AVisualLoggerRenderingActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		FVisualLoggerDatabase::Get().GetEvents().OnItemSelectionChanged.AddUObject(this, &AVisualLoggerRenderingActor::OnItemSelectionChanged);
		FVisualLoggerDatabase::Get().GetEvents().OnRowSelectionChanged.AddUObject(this, &AVisualLoggerRenderingActor::ObjectSelectionChanged);
		FVisualLoggerDatabase::Get().GetEvents().OnRowChangedVisibility.AddUObject(this, &AVisualLoggerRenderingActor::ObjectVisibilityChanged);

		FLogVisualizer::Get().GetEvents().OnFiltersChanged.AddUObject(this, &AVisualLoggerRenderingActor::OnFiltersChanged);
	}

#if VLOG_TEST_DEBUG_RENDERING
	AddDebugRendering();
#endif
}

AVisualLoggerRenderingActor::~AVisualLoggerRenderingActor()
{
	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		FVisualLoggerDatabase::Get().GetEvents().OnItemSelectionChanged.RemoveAll(this);
		FVisualLoggerDatabase::Get().GetEvents().OnRowSelectionChanged.RemoveAll(this);
		FVisualLoggerDatabase::Get().GetEvents().OnRowChangedVisibility.RemoveAll(this);
		FLogVisualizer::Get().GetEvents().OnFiltersChanged.RemoveAll(this);
	}
}

void AVisualLoggerRenderingActor::ObjectVisibilityChanged(const FName& RowName)
{
	if (DebugShapesPerRow.Contains(RowName) == false)
	{
		return;
	}

	FTimelineDebugShapes& ShapesCache = DebugShapesPerRow[RowName];
	ShapesCache.Reset();

	if (FVisualLoggerDatabase::Get().IsRowVisible(RowName))
	{
		const FVisualLoggerDBRow &DBRow = FVisualLoggerDatabase::Get().GetRowByName(RowName);
		if (DBRow.GetCurrentItemIndex() != INDEX_NONE)
		{
			GetDebugShapes(DBRow.GetItems()[DBRow.GetCurrentItemIndex()].Entry, true, ShapesCache);
		}
	}

	MarkComponentsRenderStateDirty();
}

void AVisualLoggerRenderingActor::ObjectSelectionChanged(const TArray<FName>& Selection)
{
	if (Selection.Num() > 0)
	{
		for (auto CurrentName : Selection)
		{
			if (DebugShapesPerRow.Contains(CurrentName) == false)
			{
				DebugShapesPerRow.Add(CurrentName);
				FVisualLoggerDBRow &DBRow = FVisualLoggerDatabase::Get().GetRowByName(CurrentName);
				FTimelineDebugShapes& ShapesCache = DebugShapesPerRow[CurrentName];
				for (const auto &CurrentEntry : DBRow.GetItems())
				{
					if (CurrentEntry.Entry.Location != FVector::ZeroVector)
					{
						ShapesCache.LogEntriesPath.Add(CurrentEntry.Entry.Location);
					}
				}
			}
		}

		for (TMap<FName, FTimelineDebugShapes>::TIterator It(DebugShapesPerRow); It; ++It)
		{
			if (Selection.Find(It->Key) == INDEX_NONE)
			{
				It.RemoveCurrent();
			}
		}
	}
	else
	{
		DebugShapesPerRow.Reset();
	}
	CachedRowSelection = Selection;
	MarkComponentsRenderStateDirty();
}

void AVisualLoggerRenderingActor::OnItemSelectionChanged(const FVisualLoggerDBRow& DBRow, const int32 ItemIndex)
{
	const FName RowName = DBRow.GetOwnerName();

	const TMap<FName, FVisualLogExtensionInterface*>& AllExtensions = FVisualLogger::Get().GetAllExtensions();
	for (auto& Extension : AllExtensions)
	{
		Extension.Value->DrawData(FVisualLoggerEditorInterface::Get(), nullptr);
	}

	if (DebugShapesPerRow.Contains(RowName) == false)
	{
		return;
	}

	if (FVisualLoggerDatabase::Get().IsRowVisible(RowName) == false || DBRow.GetItems().IsValidIndex(ItemIndex) == false)
	{
		return;
	}

	FTimelineDebugShapes& ShapesCache = DebugShapesPerRow[RowName];
	ShapesCache.Reset();
	const TArray<FVisualLogDevice::FVisualLogEntryItem>& Entries = DBRow.GetItems();
	const int32 CurrentItemIndex = DBRow.GetCurrentItemIndex();
	GetDebugShapes(Entries[CurrentItemIndex].Entry, true, ShapesCache);

	MarkComponentsRenderStateDirty();
}

void AVisualLoggerRenderingActor::ResetRendering()
{
	CachedRowSelection.Reset();
	DebugShapesPerRow.Reset();
	MarkComponentsRenderStateDirty();
}

void AVisualLoggerRenderingActor::OnFiltersChanged()
{
	const TMap<FName, FVisualLogExtensionInterface*>& AllExtensions = FVisualLogger::Get().GetAllExtensions();
	for (auto& Extension : AllExtensions)
	{
		Extension.Value->DrawData(FVisualLoggerEditorInterface::Get(), nullptr);
	}

	DebugShapesPerRow.Reset();
	const TArray<FName>& RowNames = FVisualLoggerDatabase::Get().GetSelectedRows();
	for (FName CurrentName : RowNames)
	{
		FVisualLoggerDBRow& DBRow = FVisualLoggerDatabase::Get().GetRowByName(CurrentName);
		FTimelineDebugShapes& ShapesCache = DebugShapesPerRow.FindOrAdd(CurrentName);
		ShapesCache.Reset();
		if (DBRow.GetCurrentItemIndex() != INDEX_NONE)
		{
			GetDebugShapes(DBRow.GetCurrentItem().Entry, true, ShapesCache);
		}
	}
	MarkComponentsRenderStateDirty();
}

#if VLOG_TEST_DEBUG_RENDERING
void AVisualLoggerRenderingActor::AddDebugRendering()
{
	const float Thickness = 2;

	{
		const FVector BoxExtent(100, 100, 100);
		const FBox Box(FVector(128), FVector(300));
		TestDebugShapes.Boxes.Add(FDebugRenderSceneProxy::FDebugBox(Box, FColor::Red));
		FTransform Trans;
		Trans.SetRotation(FQuat::MakeFromEuler(FVector(0.1, 0.2, 1.2)));
		TestDebugShapes.Boxes.Add(FDebugRenderSceneProxy::FDebugBox(Box, FColor::Red, Trans));
	}
	{
		const FVector Orgin = FVector(400,0,128);
		const FVector Direction = FVector(0,0,1);
		const float Length = 300;

		FVector YAxis, ZAxis;
		Direction.FindBestAxisVectors(YAxis, ZAxis);
		TestDebugShapes.Cones.Add(FDebugRenderSceneProxy::FCone(FScaleMatrix(FVector(Length)) * FMatrix(Direction, YAxis, ZAxis, Orgin), 30, 30, FColor::Blue));
	}
	{
		const FVector Start = FVector(700, 0, 128);
		const FVector End = FVector(700, 0, 128+300);
		const float Radius = 200;
		const float HalfHeight = 150;
		TestDebugShapes.Cylinders.Add(FDebugRenderSceneProxy::FWireCylinder(Start + FVector(0, 0, HalfHeight), (End - Start).GetSafeNormal(), Radius, HalfHeight, FColor::Magenta));
	}

	{
		const FVector Base = FVector(1000, 0, 128);
		const float HalfHeight = 150;
		const float Radius = 50;
		const FQuat Rotation = FQuat::Identity;

		const FMatrix Axes = FQuatRotationTranslationMatrix(Rotation, FVector::ZeroVector);
		const FVector XAxis = Axes.GetScaledAxis(EAxis::X);
		const FVector YAxis = Axes.GetScaledAxis(EAxis::Y);
		const FVector ZAxis = Axes.GetScaledAxis(EAxis::Z);

		TestDebugShapes.Capsules.Add(FDebugRenderSceneProxy::FCapsule(Base, Radius, XAxis, YAxis, ZAxis, HalfHeight, FColor::Yellow));
	}
	{
		const float Radius = 50;
		TestDebugShapes.Points.Add(FDebugRenderSceneProxy::FSphere(10, FVector(1300, 0, 128), FColor::White));
	}
}
#endif

void AVisualLoggerRenderingActor::IterateDebugShapes(const TFunction<void(const AVisualLoggerRenderingActorBase::FTimelineDebugShapes& Shapes)> Callback)
{
	for (auto& CurrentShapes : DebugShapesPerRow)
	{
		Callback(CurrentShapes.Value);
	}

#if VLOG_TEST_DEBUG_RENDERING
	Callback(TestDebugShapes);
#endif
}

bool AVisualLoggerRenderingActor::MatchCategoryFilters(const FName& CategoryName, ELogVerbosity::Type Verbosity) const
{
	return FVisualLoggerFilters::Get().ShouldDisplayCategory(CategoryName, Verbosity);
}
