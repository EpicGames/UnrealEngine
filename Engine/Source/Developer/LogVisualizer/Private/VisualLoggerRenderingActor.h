// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VisualLoggerRenderingActorBase.h"
#include "VisualLoggerRenderingActor.generated.h"

#define UE_API LOGVISUALIZER_API

// enable adding some hard coded shapes to the VisualLoggerRenderingActor for testing
#define VLOG_TEST_DEBUG_RENDERING 0

class UPrimitiveComponent;
struct FVisualLoggerDBRow;

/**
*	Transient actor used to draw visual logger data on level
*/

DECLARE_MULTICAST_DELEGATE_OneParam(FOnSelectionChanged, class AActor*);

UCLASS(MinimalAPI, config = Engine, NotBlueprintable, Transient, notplaceable, AdvancedClassDisplay)
class AVisualLoggerRenderingActor : public AVisualLoggerRenderingActorBase 
{
public:
	GENERATED_UCLASS_BODY()
	UE_API virtual ~AVisualLoggerRenderingActor();
	UE_API void ResetRendering();
	UE_API void ObjectVisibilityChanged(const FName& RowName);
	UE_API void ObjectSelectionChanged(const TArray<FName>& Selection);
	UE_API void OnItemSelectionChanged(const FVisualLoggerDBRow& BDRow, int32 ItemIndex);

	UE_API virtual void IterateDebugShapes(TFunction<void(const FTimelineDebugShapes&) > Callback) override;
	UE_API virtual bool MatchCategoryFilters(const FName& CategoryName, ELogVerbosity::Type Verbosity) const override;
private:
	UE_API void OnFiltersChanged();

	TArray<FName> CachedRowSelection;
	TMap<FName, FTimelineDebugShapes> DebugShapesPerRow;

#if VLOG_TEST_DEBUG_RENDERING
	UE_API void AddDebugRendering();
	FTimelineDebugShapes TestDebugShapes;
#endif
};

#undef UE_API
