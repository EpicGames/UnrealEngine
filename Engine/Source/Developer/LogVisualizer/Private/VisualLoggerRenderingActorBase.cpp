// Copyright Epic Games, Inc. All Rights Reserved.

#include "VisualLoggerRenderingActorBase.h"
#include "DynamicMeshBuilder.h"
#include "LogVisualizerSettings.h"
#include "Materials/Material.h"
#if WITH_EDITOR
#include "GeomTools.h"
#endif // WITH_EDITOR
#include "VisualLoggerRenderingComponent.h"
#include "VisualLogger/VisualLogger.h"
#include "AI/Navigation/NavigationTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VisualLoggerRenderingActorBase)

namespace FDebugDrawing
{
	const FVector NavOffset(0., 0., 15.);
}

class UVisualLoggerRenderingComponent;
class FVisualLoggerSceneProxy final : public FDebugRenderSceneProxy
{
public:
	virtual SIZE_T GetTypeHash() const override
	{
		static size_t UniquePointer;
		return reinterpret_cast<size_t>(&UniquePointer);
	}

	FVisualLoggerSceneProxy(const UVisualLoggerRenderingComponent* InComponent)
		: FDebugRenderSceneProxy(InComponent)
	{
		DrawType = SolidAndWireMeshes;
		ViewFlagName = TEXT("VisLog");
		ViewFlagIndex = uint32(FEngineShowFlags::FindIndexByName(*ViewFlagName));
		bWantsSelectionOutline = false;
		FarClippingDistance = InComponent->GetFarClippingDistance();
	}

	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
	{
		FPrimitiveViewRelevance Result;
		Result.bDrawRelevance = IsShown(View);
		Result.bDynamicRelevance = true;
		// ideally the TranslucencyRelevance should be filled out by the material, here we do it conservative
		Result.bSeparateTranslucency = Result.bNormalTranslucency = IsShown(View) && GIsEditor;
		return Result;
	}

	virtual uint32 GetMemoryFootprint(void) const override { return sizeof(*this) + GetAllocatedSize(); }
};

UVisualLoggerRenderingComponent::UVisualLoggerRenderingComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UVisualLoggerRenderingComponent::SetFarClippingDistance(double Distance)
{
	if (FarClippingDistance != Distance)
	{
		FarClippingDistance = Distance;
		MarkRenderStateDirty();
	}
}


FDebugRenderSceneProxy* UVisualLoggerRenderingComponent::CreateDebugSceneProxy()
{
	AVisualLoggerRenderingActorBase* RenderingActor = Cast<AVisualLoggerRenderingActorBase>(GetOuter());
	if (RenderingActor == nullptr)
	{
		return nullptr;
	}

	ULogVisualizerSettings *Settings = ULogVisualizerSettings::StaticClass()->GetDefaultObject<ULogVisualizerSettings>();
	FVisualLoggerSceneProxy *VLogSceneProxy = new FVisualLoggerSceneProxy(this);
	VLogSceneProxy->SolidMeshMaterial = Settings->GetDebugMeshMaterial();

	RenderingActor->IterateDebugShapes([VLogSceneProxy] (const AVisualLoggerRenderingActorBase::FTimelineDebugShapes& Shapes)
		{
			VLogSceneProxy->Spheres.Append(Shapes.Points);
			VLogSceneProxy->Lines.Append(Shapes.Lines);
			VLogSceneProxy->Circles.Append(Shapes.Circles);
			VLogSceneProxy->Boxes.Append(Shapes.Boxes);
			VLogSceneProxy->Meshes.Append(Shapes.Meshes);
			VLogSceneProxy->Cones.Append(Shapes.Cones);
			VLogSceneProxy->Texts.Append(Shapes.Texts);
			VLogSceneProxy->Cylinders.Append(Shapes.Cylinders);
			VLogSceneProxy->ArrowLines.Append(Shapes.Arrows);
			VLogSceneProxy->Capsules.Append(Shapes.Capsules);
			VLogSceneProxy->CoordinateSystems.Append(Shapes.CoordinateSystems);
		});

	return VLogSceneProxy;
}

FBoxSphereBounds UVisualLoggerRenderingComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	FBox MyBounds;
	MyBounds.Init();

	MyBounds = FBox(FVector(-HALF_WORLD_MAX, -HALF_WORLD_MAX, -HALF_WORLD_MAX), FVector(HALF_WORLD_MAX, HALF_WORLD_MAX, HALF_WORLD_MAX));

	return MyBounds;
}

AVisualLoggerRenderingActorBase::AVisualLoggerRenderingActorBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	USceneComponent* SceneComponent = CreateDefaultSubobject<USceneComponent>(TEXT("SceneComp"));
	RootComponent = SceneComponent;

	RenderingComponent = CreateDefaultSubobject<UVisualLoggerRenderingComponent>(TEXT("RenderingComponent"));
}

AVisualLoggerRenderingActorBase::~AVisualLoggerRenderingActorBase()
{
}

void AVisualLoggerRenderingActorBase::SetFarClippingDistance(double Distance)
{
	if (RenderingComponent != nullptr)
	{
		RenderingComponent->SetFarClippingDistance(Distance);
	}
}

double AVisualLoggerRenderingActorBase::GetFarClippingDistance() const
{
	return (RenderingComponent != nullptr) ? RenderingComponent->GetFarClippingDistance() : 0.0;
}

namespace
{
	static bool IsPolygonWindingCorrect(const TArray<FVector>& Verts)
	{
		// this will work only for convex polys, but we're assuming that all logged polygons are convex in the first place
		if (Verts.Num() >= 3)
		{
			const FVector SurfaceNormal = FVector::CrossProduct(Verts[1] - Verts[0], Verts[2] - Verts[0]);
			const double TestDot = FVector::DotProduct(SurfaceNormal, FVector::UpVector);
			return TestDot > 0.;
		}

		return false;
	}

	static void GetPolygonMesh(const FVisualLogShapeElement* ElementToDraw, FDebugRenderSceneProxy::FMesh& TestMesh, const FVector3f& VertexOffset = FVector3f::ZeroVector)
	{
		TestMesh.Color = ElementToDraw->GetFColor();

		FClipSMPolygon InPoly(ElementToDraw->Points.Num());
		InPoly.FaceNormal = FVector::UpVector;

		const bool bHasCorrectWinding = IsPolygonWindingCorrect(ElementToDraw->Points);
		if (bHasCorrectWinding)
		{
			for (int32 Index = 0; Index < ElementToDraw->Points.Num(); Index++)
			{
				FClipSMVertex v1;
				v1.Pos = (FVector3f)ElementToDraw->Points[Index];
				InPoly.Vertices.Add(v1);
			}
		}
		else
		{
			for (int32 Index = ElementToDraw->Points.Num() - 1; Index >= 0; Index--)
			{
				FClipSMVertex v1;
				v1.Pos = (FVector3f)ElementToDraw->Points[Index];
				InPoly.Vertices.Add(v1);
			}
		}

		TArray<FClipSMTriangle> OutTris;
		
		const bool bTriangulated = FGeomTools::TriangulatePoly(OutTris, InPoly, false);
		if (bTriangulated)
		{
			int32 LastIndex = 0;

			FGeomTools::RemoveRedundantTriangles(OutTris);
			for (const auto& CurrentTri : OutTris)
			{
				TestMesh.Vertices.Add(FDynamicMeshVertex(CurrentTri.Vertices[0].Pos + VertexOffset));
				TestMesh.Vertices.Add(FDynamicMeshVertex(CurrentTri.Vertices[1].Pos + VertexOffset));
				TestMesh.Vertices.Add(FDynamicMeshVertex(CurrentTri.Vertices[2].Pos + VertexOffset));
				TestMesh.Indices.Add(LastIndex++);
				TestMesh.Indices.Add(LastIndex++);
				TestMesh.Indices.Add(LastIndex++);
			}
		}
	}
}

void AVisualLoggerRenderingActorBase::GetDebugShapes(const FVisualLogEntry& InEntry, bool bAddEntryLocationPointer, AVisualLoggerRenderingActorBase::FTimelineDebugShapes& DebugShapes)
{
	const FVisualLogEntry* Entry = &InEntry;
	const FVisualLogShapeElement* ElementToDraw = Entry->ElementsToDraw.GetData();
	const int32 ElementsCount = Entry->ElementsToDraw.Num();

	if (bAddEntryLocationPointer)
	{
		constexpr float Length = 100;
		const FVector DirectionNorm = FVector::UpVector;
		FVector YAxis, ZAxis;
		DirectionNorm.FindBestAxisVectors(YAxis, ZAxis);
		DebugShapes.Cones.Add(FDebugRenderSceneProxy::FCone(FScaleMatrix(FVector(Length)) * FMatrix(DirectionNorm, YAxis, ZAxis, Entry->Location), 5, 5, FColor::Red));
	}

	if (DebugShapes.LogEntriesPath.Num())
	{
		FVector Location = DebugShapes.LogEntriesPath[0];
		for (int32 Index = 1; Index < DebugShapes.LogEntriesPath.Num(); ++Index)
		{
			const FVector CurrentLocation = DebugShapes.LogEntriesPath[Index];
			DebugShapes.Lines.Add(FDebugRenderSceneProxy::FDebugLine(Location, CurrentLocation, FColor(160, 160, 240), 2.0));
			Location = CurrentLocation;
		}
	}

	for (int32 ElementIndex = 0; ElementIndex < ElementsCount; ++ElementIndex, ++ElementToDraw)
	{
		if (!MatchCategoryFilters(ElementToDraw->Category, ElementToDraw->Verbosity))
		{
			continue;
		}

		const FVector3f CorridorOffset = (FVector3f)FDebugDrawing::NavOffset * 1.25f;
		const FColor Color = ElementToDraw->GetFColor();
		const EVisualLoggerShapeElement ElementType = ElementToDraw->GetType();
		// The wire elements force the draw type : 
		const FDebugRenderSceneProxy::EDrawType DrawTypeOverride = (
			(ElementType == EVisualLoggerShapeElement::WireBox) 
			|| (ElementType == EVisualLoggerShapeElement::WireSphere)
			|| (ElementType == EVisualLoggerShapeElement::WireCapsule)
			|| (ElementType == EVisualLoggerShapeElement::WireCone)
			|| (ElementType == EVisualLoggerShapeElement::WireCylinder)
			|| (ElementType == EVisualLoggerShapeElement::WireCircle)) ? FDebugRenderSceneProxy::EDrawType::WireMesh : FDebugRenderSceneProxy::EDrawType::Invalid;

		switch (ElementType)
		{
		case EVisualLoggerShapeElement::SinglePoint:
		case EVisualLoggerShapeElement::Sphere:
		case EVisualLoggerShapeElement::WireSphere:
		{
			const float Radius = float(ElementToDraw->Radius);
			const bool bDrawLabel = (ElementToDraw->Description.IsEmpty() == false);
			const int32 NumPoints = ElementToDraw->Points.Num();

			for (int32 Index = 0; Index < NumPoints; ++Index)
			{
				const FVector& Point = ElementToDraw->Points[Index];
				DebugShapes.Points.Add(FDebugRenderSceneProxy::FSphere(Radius, Point, Color, DrawTypeOverride));
				if (bDrawLabel)
				{
					const FString PrintString = NumPoints == 1 ? ElementToDraw->Description : FString::Printf(TEXT("%s_%d"), *ElementToDraw->Description, Index);
					DebugShapes.Texts.Add(FDebugRenderSceneProxy::FText3d(PrintString, Point, Color.WithAlpha(255)));
				}
			}
		}
			break;
		case EVisualLoggerShapeElement::Polygon:
		{
			FDebugRenderSceneProxy::FMesh TestMesh;
			GetPolygonMesh(ElementToDraw, TestMesh, CorridorOffset);
			DebugShapes.Meshes.Add(TestMesh);

			for (int32 VIdx = 0; VIdx < ElementToDraw->Points.Num(); VIdx++)
			{
				DebugShapes.Lines.Add(FDebugRenderSceneProxy::FDebugLine(
					ElementToDraw->Points[VIdx] + (FVector)CorridorOffset,
					ElementToDraw->Points[(VIdx + 1) % ElementToDraw->Points.Num()] + (FVector)CorridorOffset,
					FColor::Cyan,
					2)
					);
			}
		}
			break;
		case EVisualLoggerShapeElement::Mesh:
		{
			struct FHeaderData
			{
				int32 VerticesNum, FacesNum;
				FHeaderData(const FVector& InVector) : VerticesNum(static_cast<int32>(InVector.X)), FacesNum(static_cast<int32>(InVector.Y)) {}
			};
			const FHeaderData HeaderData(ElementToDraw->Points[0]);

			FDebugRenderSceneProxy::FMesh TestMesh;
			TestMesh.Color = ElementToDraw->GetFColor();
			int32 StartIndex = 1;
			int32 EndIndex = StartIndex + HeaderData.VerticesNum;
			for (int32 VIdx = StartIndex; VIdx < EndIndex; VIdx++)
			{
				const FVector& CurrentPoint = ElementToDraw->Points[VIdx];
				TestMesh.Vertices.Add(FVector3f(CurrentPoint));
				TestMesh.Box += CurrentPoint;
			}


			StartIndex = EndIndex;
			EndIndex = StartIndex + HeaderData.FacesNum;
			for (int32 VIdx = StartIndex; VIdx < EndIndex; VIdx++)
			{
				const FVector &CurrentFace = ElementToDraw->Points[VIdx];
				TestMesh.Indices.Add(static_cast<uint32>(CurrentFace.X));
				TestMesh.Indices.Add(static_cast<uint32>(CurrentFace.Y));
				TestMesh.Indices.Add(static_cast<uint32>(CurrentFace.Z));
			}
			DebugShapes.Meshes.Add(TestMesh);
		}
			break;
		case EVisualLoggerShapeElement::Segment:
		{
			const float Thickness = float(ElementToDraw->Thickness);
			const bool bDrawLabel = (ElementToDraw->Description.IsEmpty() == false);
			const FVector* Location = ElementToDraw->Points.GetData();
			const int32 NumPoints = ElementToDraw->Points.Num();

			for (int32 Index = 0; Index + 1 < NumPoints; Index += 2, Location += 2)
			{
				DebugShapes.Lines.Add(FDebugRenderSceneProxy::FDebugLine(*Location, *(Location + 1), Color, Thickness));

				if (bDrawLabel)
				{
					const FString PrintString = NumPoints == 2 ? ElementToDraw->Description : FString::Printf(TEXT("%s_%d"), *ElementToDraw->Description, Index / 2);
					DebugShapes.Texts.Add(FDebugRenderSceneProxy::FText3d(PrintString, (*Location + (*(Location + 1) - *Location) / 2), Color.WithAlpha(255)));
				}
			}
		}
			break;
		case EVisualLoggerShapeElement::Path:
		{
			const float Thickness = float(ElementToDraw->Thickness);
			FVector Location = ElementToDraw->Points[0];
			for (int32 Index = 1; Index < ElementToDraw->Points.Num(); ++Index)
			{
				const FVector CurrentLocation = ElementToDraw->Points[Index];
				DebugShapes.Lines.Add(FDebugRenderSceneProxy::FDebugLine(Location, CurrentLocation, Color, Thickness));
				Location = CurrentLocation;
			}
		}
			break;
		case EVisualLoggerShapeElement::Box:
		case EVisualLoggerShapeElement::WireBox:
		{
			const float Thickness = float(ElementToDraw->Thickness);
			const bool bDrawLabel = (ElementToDraw->Description.IsEmpty() == false);
			const FVector* BoxExtent = ElementToDraw->Points.GetData();
			const int32 NumPoints = ElementToDraw->Points.Num();

			FTransform Transform(ElementToDraw->TransformationMatrix);
			for (int32 Index = 0; Index + 1 < NumPoints; Index += 2, BoxExtent += 2)
			{
				const FBox Box = FBox(*BoxExtent, *(BoxExtent + 1));
				DebugShapes.Boxes.Add(FDebugRenderSceneProxy::FDebugBox(Box, Color, Transform, DrawTypeOverride, Thickness));

				if (bDrawLabel)
				{
					const FString PrintString = NumPoints == 2 ? ElementToDraw->Description : FString::Printf(TEXT("%s_%d"), *ElementToDraw->Description, Index / 2);
					DebugShapes.Texts.Add(FDebugRenderSceneProxy::FText3d(PrintString, Transform.TransformPosition(Box.GetCenter()), Color.WithAlpha(255)));
				}
			}
		}
			break;
		case EVisualLoggerShapeElement::Cone:
		case EVisualLoggerShapeElement::WireCone:
		{
			const float Thickness = float(ElementToDraw->Thickness);
			const bool bDrawLabel = ElementToDraw->Description.IsEmpty() == false;
			for (int32 Index = 0; Index + 2 < ElementToDraw->Points.Num(); Index += 3)
			{
				const FVector Origin = ElementToDraw->Points[Index];
				const FVector Direction = ElementToDraw->Points[Index + 1].GetSafeNormal();
				const FVector Angles = ElementToDraw->Points[Index + 2];
				const double Length = Angles.X;

				FVector YAxis, ZAxis;
				Direction.FindBestAxisVectors(YAxis, ZAxis);
				DebugShapes.Cones.Add(FDebugRenderSceneProxy::FCone(FScaleMatrix(FVector(Length)) * FMatrix(Direction, YAxis, ZAxis, Origin),
					static_cast<float>(Angles.Y), static_cast<float>(Angles.Z), Color, DrawTypeOverride, Thickness));

				if (bDrawLabel)
				{
					DebugShapes.Texts.Add(FDebugRenderSceneProxy::FText3d(ElementToDraw->Description, Origin, Color.WithAlpha(255)));
				}
			}
		}
			break;
		case EVisualLoggerShapeElement::Cylinder:
		case EVisualLoggerShapeElement::WireCylinder:
		{
			const float Thickness = float(ElementToDraw->Thickness);
			const bool bDrawLabel = ElementToDraw->Description.IsEmpty() == false;
			for (int32 Index = 0; Index + 2 < ElementToDraw->Points.Num(); Index += 3)
			{
				const FVector Start = ElementToDraw->Points[Index];
				const FVector End = ElementToDraw->Points[Index + 1];
				const FVector OtherData = ElementToDraw->Points[Index + 2];
				const float HalfHeight = FloatCastChecked<float>(0.5 * (End - Start).Size(), UE::LWC::DefaultFloatPrecision);
				const FVector Center = 0.5 * (Start + End);
				DebugShapes.Cylinders.Add(FDebugRenderSceneProxy::FWireCylinder(Center
					, (End - Start).GetSafeNormal()
					, static_cast<float>(OtherData.X)
					, HalfHeight, Color, DrawTypeOverride, Thickness)); // Base parameter is the center of the cylinder
				if (bDrawLabel)
				{
					DebugShapes.Texts.Add(FDebugRenderSceneProxy::FText3d(ElementToDraw->Description, Center, Color.WithAlpha(255)));
				}
			}
		}
			break;
		case EVisualLoggerShapeElement::Capsule:
		case EVisualLoggerShapeElement::WireCapsule:
		{
			const float Thickness = float(ElementToDraw->Thickness);
			const bool bDrawLabel = ElementToDraw->Description.IsEmpty() == false;
			for (int32 Index = 0; Index + 2 < ElementToDraw->Points.Num(); Index += 3)
			{
				const FVector Base = ElementToDraw->Points[Index + 0];
				const FVector FirstData = ElementToDraw->Points[Index + 1];
				const FVector SecondData = ElementToDraw->Points[Index + 2];
				const float HalfHeight = static_cast<float>(FirstData.X);
				const float Radius = static_cast<float>(FirstData.Y);
				const FQuat Rotation = FQuat(FirstData.Z, SecondData.X, SecondData.Y, SecondData.Z);

				const FMatrix Axes = FQuatRotationTranslationMatrix(Rotation, FVector::ZeroVector);
				const FVector XAxis = Axes.GetScaledAxis(EAxis::X);
				const FVector YAxis = Axes.GetScaledAxis(EAxis::Y);
				const FVector ZAxis = Axes.GetScaledAxis(EAxis::Z);

				DebugShapes.Capsules.Add(FDebugRenderSceneProxy::FCapsule(Base, Radius, XAxis, YAxis, ZAxis, HalfHeight, Color, DrawTypeOverride, Thickness));
				if (bDrawLabel)
				{
					DebugShapes.Texts.Add(FDebugRenderSceneProxy::FText3d(ElementToDraw->Description, Base + HalfHeight * FVector::UpVector, Color.WithAlpha(255)));
				}
			}
		}
			break;
		case EVisualLoggerShapeElement::NavAreaMesh:
		{
			if (ElementToDraw->Points.Num() == 0)
			{
				continue;
			}

			struct FHeaderData
			{
				float MinZ, MaxZ;
				FHeaderData(const FVector& InVector) : MinZ(FloatCastChecked<float>(InVector.X, UE::LWC::DefaultFloatPrecision)), MaxZ(FloatCastChecked<float>(InVector.Y, UE::LWC::DefaultFloatPrecision)) {}
			};
			const FHeaderData HeaderData(ElementToDraw->Points[0]);

			TArray<FVector> AreaMeshPoints = ElementToDraw->Points;
			AreaMeshPoints.RemoveAt(0, EAllowShrinking::No);
			AreaMeshPoints.Add(ElementToDraw->Points[1]);
			TArray<FVector> Vertices;
			TNavStatArray<FVector> Faces;
			int32 CurrentIndex = 0;
			FDebugRenderSceneProxy::FMesh TestMesh;
			TestMesh.Color = ElementToDraw->GetFColor();
			TestMesh.Vertices.Reserve((AreaMeshPoints.Num() - 1) * 6);
			TestMesh.Indices.Reserve((AreaMeshPoints.Num() - 1) * 6);

			for (int32 PointIndex = 0; PointIndex < AreaMeshPoints.Num() - 1; PointIndex++)
			{
				FVector Point = AreaMeshPoints[PointIndex];
				FVector NextPoint = AreaMeshPoints[PointIndex + 1];

				
				// LWC_TODO These won't work well with very large coordinates!
				FVector3f P1(UE::LWC::NarrowWorldPositionChecked(Point.X, Point.Y, HeaderData.MinZ));
				FVector3f P2(UE::LWC::NarrowWorldPositionChecked(Point.X, Point.Y, HeaderData.MaxZ));
				FVector3f P3(UE::LWC::NarrowWorldPositionChecked(NextPoint.X, NextPoint.Y, HeaderData.MinZ));
				FVector3f P4(UE::LWC::NarrowWorldPositionChecked(NextPoint.X, NextPoint.Y, HeaderData.MaxZ));

				TestMesh.Vertices.Add(P1); 
				TestMesh.Vertices.Add(P2); 
				TestMesh.Vertices.Add(P3);

				TestMesh.Indices.Add(CurrentIndex++);
				TestMesh.Indices.Add(CurrentIndex++);
				TestMesh.Indices.Add(CurrentIndex++);

				TestMesh.Vertices.Add(P3); 
				TestMesh.Vertices.Add(P2); 
				TestMesh.Vertices.Add(P4);

				TestMesh.Indices.Add(CurrentIndex++);
				TestMesh.Indices.Add(CurrentIndex++);
				TestMesh.Indices.Add(CurrentIndex++);
			}
			DebugShapes.Meshes.Add(TestMesh);

			{
				FDebugRenderSceneProxy::FMesh PolygonMesh;
				FVisualLogShapeElement PolygonToDraw(EVisualLoggerShapeElement::Polygon);
				PolygonToDraw.SetColor(ElementToDraw->GetFColor());
				PolygonToDraw.Points.Reserve(AreaMeshPoints.Num());
				PolygonToDraw.Points = AreaMeshPoints;
				GetPolygonMesh(&PolygonToDraw, PolygonMesh, FVector3f(0., 0., HeaderData.MaxZ));
				DebugShapes.Meshes.Add(PolygonMesh);
			}

			for (int32 VIdx = 0; VIdx < AreaMeshPoints.Num(); VIdx++)
			{
				DebugShapes.Lines.Add(FDebugRenderSceneProxy::FDebugLine(
					AreaMeshPoints[VIdx] + FVector(0., 0., HeaderData.MaxZ),
					AreaMeshPoints[(VIdx + 1) % AreaMeshPoints.Num()] + FVector(0., 0., HeaderData.MaxZ),
					ElementToDraw->GetFColor(),
					2)
					);
			}

		}
			break;

		case EVisualLoggerShapeElement::Arrow:
		{
			const bool bDrawLabel = (ElementToDraw->Description.IsEmpty() == false);
			const FVector* Location = ElementToDraw->Points.GetData();
			const int32 NumPoints = ElementToDraw->Points.Num();
			const float Mag = float(ElementToDraw->Mag);

			for (int32 Index = 0; Index + 1 < NumPoints; Index += 2, Location += 2)
			{
				DebugShapes.Arrows.Add(FDebugRenderSceneProxy::FArrowLine(*Location, *(Location + 1), Color, Mag));

				if (bDrawLabel)
				{
					const FString PrintString = NumPoints == 2 ? ElementToDraw->Description : FString::Printf(TEXT("%s_%d"), *ElementToDraw->Description, Index / 2);
					DebugShapes.Texts.Add(FDebugRenderSceneProxy::FText3d(PrintString, (*Location + (*(Location + 1) - *Location) / 2), Color.WithAlpha(255)));
				}
			}
		}
			break;

		case EVisualLoggerShapeElement::Circle:
		case EVisualLoggerShapeElement::WireCircle:
		{
			const float Thickness = float(ElementToDraw->Thickness);
			const bool bDrawLabel = ElementToDraw->Description.IsEmpty() == false;
			const int32 NumPoints = ElementToDraw->Points.Num();
			for (int32 Index = 0; Index + 2 < NumPoints; Index += 3)
			{
				const FVector Center = ElementToDraw->Points[Index + 0];
				const FVector UpAxis = ElementToDraw->Points[Index + 1];
				const double Radius = ElementToDraw->Points[Index + 2].X;

				DebugShapes.Circles.Add(FDebugRenderSceneProxy::FCircle(Center, UpAxis.GetSafeNormal(), Radius, Color, DrawTypeOverride, Thickness)); 
				
				if (bDrawLabel)
				{
					const FString PrintString = NumPoints == 3 ? ElementToDraw->Description : FString::Printf(TEXT("%s_%d"), *ElementToDraw->Description, Index / 3);
					DebugShapes.Texts.Add(FDebugRenderSceneProxy::FText3d(PrintString, Center, Color.WithAlpha(255)));
				}
			}
		}
			break;
		case EVisualLoggerShapeElement::CoordinateSystem:
		{
			const float Thickness = float(ElementToDraw->Thickness);
			const bool bDrawLabel = ElementToDraw->Description.IsEmpty() == false;
			const int32 NumPoints = ElementToDraw->Points.Num();

			for (int32 Index = 0; Index + 2 < NumPoints; Index += 3)
			{
				const FVector AxisLoc = ElementToDraw->Points[Index];
				const FRotator AxisRot = ElementToDraw->Points[Index + 1].ToOrientationRotator();
				const float Scale = ElementToDraw->Points[Index + 2].X;

				DebugShapes.CoordinateSystems.Add(FDebugRenderSceneProxy::FCoordinateSystem(AxisLoc, AxisRot, Scale, Color, Thickness));

				if (bDrawLabel)
				{
					const FString PrintString = NumPoints == 3 ? ElementToDraw->Description : FString::Printf(TEXT("%s_%d"), *ElementToDraw->Description, Index / 3);
					DebugShapes.Texts.Add(FDebugRenderSceneProxy::FText3d(PrintString, AxisLoc, Color.WithAlpha(255)));
				}
			}
		}
			break;
		case EVisualLoggerShapeElement::Invalid:
			UE_LOGF(LogVisual, Warning, "Invalid element type");
			break;

		default:
			UE_LOGF(LogVisual, Warning, "Unhandled element type: %d", int(ElementToDraw->GetType()));
		}
	}
}
