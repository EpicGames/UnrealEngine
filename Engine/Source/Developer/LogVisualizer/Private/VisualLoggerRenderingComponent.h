// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Debug/DebugDrawComponent.h"
#include "VisualLoggerRenderingComponent.generated.h"

/**
 *	Primitive component used to draw visual logger data on level
 */
UCLASS(ClassGroup = Debug)
class UVisualLoggerRenderingComponent : public UDebugDrawComponent
{
public:
	GENERATED_UCLASS_BODY()

	virtual FDebugRenderSceneProxy* CreateDebugSceneProxy() override;
	virtual FBoxSphereBounds CalcBounds(const FTransform &LocalToWorld) const override;

	// Allows to override the far clipping distance for all the log elements by this actor :
	void SetFarClippingDistance(double Distance);
	double GetFarClippingDistance() const { return FarClippingDistance; }

protected:
	double FarClippingDistance = 0.0;
};
