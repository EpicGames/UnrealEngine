// Copyright Epic Games, Inc. All Rights Reserved.

#include "VisualLoggerTimeSliderController.h"
#include "Rendering/DrawElements.h"
#include "Fonts/FontMeasure.h"
#include "Styling/CoreStyle.h"
#include "Framework/Application/SlateApplication.h"
#include "Styling/AppStyle.h"


#define LOCTEXT_NAMESPACE "TimeSlider"


namespace ScrubConstants
{
	/** The minimum amount of pixels between each major ticks on the widget */
	const int32 MinPixelsPerDisplayTick = 5;

	/**The smallest number of units between between major tick marks */
	const float MinDisplayTickSpacing = 0.001f;
}


/** Utility struct for converting between scrub range space and local/absolute screen space */
struct FVisualLoggerTimeSliderController::FScrubRangeToScreen
{
	FVector2f WidgetSize;

	TRange<double> ViewInput;
	double ViewInputRange;
	double PixelsPerInput;

	FScrubRangeToScreen(TRange<double> InViewInput, const FVector2f& InWidgetSize )
	{
		WidgetSize = InWidgetSize;

		ViewInput = InViewInput;
		ViewInputRange = ViewInput.Size<double>();
		PixelsPerInput = ViewInputRange > 0. ? ( WidgetSize.X / ViewInputRange ) : 0.;
	}

	/** Local Widget Space -> Curve Input domain. */
	double LocalXToInput(float ScreenX) const
	{
		return (ScreenX/PixelsPerInput) + ViewInput.GetLowerBoundValue();
	}

	/** Curve Input domain -> local Widget Space */
	float InputToLocalX(double Input) const
	{
		return static_cast<float>((Input - ViewInput.GetLowerBoundValue()) * PixelsPerInput);
	}
};


/**
 * Gets the the next spacing value in the series 
 * to determine a good spacing value
 * E.g, .001,.005,.010,.050,.100,.500,1.000,etc
 */
static float GetNextSpacing( uint32 CurrentStep )
{
	if(CurrentStep & 0x01) 
	{
		// Odd numbers
		return FMath::Pow( 10.f, 0.5f*((float)(CurrentStep-1)) + 1.f );
	}
	else 
	{
		// Even numbers
		return 0.5f * FMath::Pow( 10.f, 0.5f*((float)(CurrentStep)) + 1.f );
	}
}

FVisualLoggerTimeSliderController::FVisualLoggerTimeSliderController(const FVisualLoggerTimeSliderArgs& InArgs)
	: TimeSliderArgs( InArgs )
	, DistanceDragged( 0. )
	, bDraggingScrubber( false )
	, bPanning( false )
{
	ScrubHandleUp = FAppStyle::GetBrush( TEXT( "Sequencer.Timeline.ScrubHandleUp" ) ); 
	ScrubHandleDown = FAppStyle::GetBrush( TEXT( "Sequencer.Timeline.ScrubHandleDown" ) );
	CursorBackground = FAppStyle::GetBrush("Sequencer.SectionArea.Background");
}

double FVisualLoggerTimeSliderController::DetermineOptimalSpacing(double InPixelsPerInput, uint32 MinTick, float MinTickSpacing) const
{
	if (InPixelsPerInput == 0.)
		return MinTickSpacing;

	uint32 CurStep = 0;

	// Start with the smallest spacing
	double Spacing = MinTickSpacing;

	while( Spacing * InPixelsPerInput < MinTick )
	{
		Spacing = MinTickSpacing * GetNextSpacing( CurStep );
		CurStep++;
	}

	return Spacing;
}

void FVisualLoggerTimeSliderController::SetTimesliderArgs(const FVisualLoggerTimeSliderArgs& InArgs)
{
	TimeSliderArgs = InArgs;
}

struct FVisualLoggerTimeSliderController::FDrawTickArgs
{
	/** Geometry of the area */
	FGeometry AllottedGeometry;
	/** Clipping rect of the area */
	FSlateRect ClippingRect;
	/** Color of each tick */
	FLinearColor TickColor;
	/** Offset in Y where to start the tick */
	float TickOffset;
	/** Height in of major ticks */
	float MajorTickHeight;
	/** Start layer for elements */
	int32 StartLayer;
	/** Draw effects to apply */
	ESlateDrawEffect DrawEffects;
	/** Whether or not to only draw major ticks */
	bool bOnlyDrawMajorTicks;
	/** Whether or not to mirror labels */
	bool bMirrorLabels;
	
};

void FVisualLoggerTimeSliderController::DrawTicks( FSlateWindowElementList& OutDrawElements, const struct FScrubRangeToScreen& RangeToScreen, FDrawTickArgs& InArgs ) const
{
	const double Spacing = DetermineOptimalSpacing( RangeToScreen.PixelsPerInput, ScrubConstants::MinPixelsPerDisplayTick, ScrubConstants::MinDisplayTickSpacing );

	// Sub divisions
	// @todo Sequencer may need more robust calculation
	const int32 Divider = 10;
	// For slightly larger halfway tick mark
	const int32 HalfDivider = Divider / 2;
	// Find out where to start from
	int32 OffsetNum = IntCastChecked<int32>(FMath::FloorToInt(RangeToScreen.ViewInput.GetLowerBoundValue() / Spacing));
	
	FSlateFontInfo SmallLayoutFont = FCoreStyle::GetDefaultFontStyle("Regular", 8);

	TArray<FVector2f> LinePoints;
	LinePoints.AddUninitialized(2);

	double Seconds = 0;
	while( (Seconds = OffsetNum*Spacing) < RangeToScreen.ViewInput.GetUpperBoundValue() )
	{
		// X position local to start of the widget area
		const float XPos = RangeToScreen.InputToLocalX( Seconds );
		uint32 AbsOffsetNum = FMath::Abs(OffsetNum);

		if ( AbsOffsetNum % Divider == 0 )
		{
			FVector2f Offset( XPos, InArgs.TickOffset );
			FVector2f TickSize( 1.0f, InArgs.MajorTickHeight );

			LinePoints[0] = FVector2f(1.0f,1.0f);
			LinePoints[1] = TickSize;

			// lines should not need anti-aliasing
			const bool bAntiAliasLines = false;

			// Draw each tick mark
			FSlateDrawElement::MakeLines(
				OutDrawElements,
				InArgs.StartLayer,
				InArgs.AllottedGeometry.ToPaintGeometry( TickSize, FSlateLayoutTransform(Offset) ),
				LinePoints,
				InArgs.DrawEffects,
				InArgs.TickColor,
				false
				);

			if( !InArgs.bOnlyDrawMajorTicks )
			{
				FString FrameString = Spacing == ScrubConstants::MinDisplayTickSpacing ? FString::Printf( TEXT("%.3f"), Seconds ) : FString::Printf( TEXT("%.2f"), Seconds );

				// Space the text between the tick mark but slightly above
				const TSharedRef< FSlateFontMeasure > FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
				const FVector2f TextSize = FontMeasureService->Measure(FrameString, SmallLayoutFont);
				const FVector2f TextOffset( XPos-(TextSize.X*0.5f), InArgs.bMirrorLabels ? TextSize.Y :  FMath::Abs( InArgs.AllottedGeometry.GetLocalSize().Y - (InArgs.MajorTickHeight+TextSize.Y) ) );

				FSlateDrawElement::MakeText(
					OutDrawElements,
					InArgs.StartLayer+1, 
					InArgs.AllottedGeometry.ToPaintGeometry( TextSize, FSlateLayoutTransform(TextOffset) ), 
					FrameString, 
					SmallLayoutFont, 
					InArgs.DrawEffects,
					InArgs.TickColor 
				);
			}
		}
		else if( !InArgs.bOnlyDrawMajorTicks )
		{
			// Compute the size of each tick mark.  If we are half way between to visible values display a slightly larger tick mark
			const float MinorTickHeight = AbsOffsetNum % HalfDivider == 0 ? 7.0f : 4.0f;

			FVector2f Offset(XPos, InArgs.bMirrorLabels ? 0.0f : FMath::Abs( InArgs.AllottedGeometry.GetLocalSize().Y - MinorTickHeight ) );
			FVector2f TickSize(1, MinorTickHeight);

			LinePoints[0] = FVector2f(1.0f,1.0f);
			LinePoints[1] = TickSize;

			const bool bAntiAlias = false;
			// Draw each sub mark
			FSlateDrawElement::MakeLines(
				OutDrawElements,
				InArgs.StartLayer,
				InArgs.AllottedGeometry.ToPaintGeometry( TickSize, FSlateLayoutTransform(Offset) ),
				LinePoints,
				InArgs.DrawEffects,
				InArgs.TickColor,
				bAntiAlias
			);
		}
		// Advance to next tick mark
		++OffsetNum;
	}
}


int32 FVisualLoggerTimeSliderController::OnPaintTimeSlider( bool bMirrorLabels, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	const bool bEnabled = bParentEnabled;
	const ESlateDrawEffect DrawEffects = bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

	TRange<double> LocalViewRange = TimeSliderArgs.ViewRange.Get();
	const double LocalViewRangeMin = LocalViewRange.GetLowerBoundValue();
	const double LocalViewRangeMax = LocalViewRange.GetUpperBoundValue();
	const double LocalSequenceLength = LocalViewRangeMax-LocalViewRangeMin;
	
	FVector2D Scale = FVector2D(1.0f,1.0f);
	if ( LocalSequenceLength > 0)
	{
		FScrubRangeToScreen RangeToScreen( LocalViewRange, AllottedGeometry.GetLocalSize() );
	
		const float MajorTickHeight = 9.f;
	
		FDrawTickArgs Args;
		Args.AllottedGeometry = AllottedGeometry;
		Args.bMirrorLabels = bMirrorLabels;
		Args.bOnlyDrawMajorTicks = false;
		Args.TickColor = FLinearColor::White;
		Args.ClippingRect = MyCullingRect;
		Args.DrawEffects = DrawEffects;
		Args.StartLayer = LayerId;
		Args.TickOffset = bMirrorLabels ? 0.f : FMath::Abs( AllottedGeometry.GetLocalSize().Y - MajorTickHeight );
		Args.MajorTickHeight = MajorTickHeight;

		DrawTicks( OutDrawElements, RangeToScreen, Args );

		const float HandleSize = 13.0f;
		float HalfSize = FMath::TruncToFloat(HandleSize/2.0f);

		// Draw the scrub handle
		const float XPos = RangeToScreen.InputToLocalX( TimeSliderArgs.ScrubPosition.Get() );

		// Draw cursor size
		const float CursorHalfSize = TimeSliderArgs.CursorSize.Get() * 0.5f;
		const int32 CursorLayer = LayerId + 2;
		const float CursorHalfLength = AllottedGeometry.GetLocalSize().X * CursorHalfSize;
		FPaintGeometry CursorGeometry = AllottedGeometry.ToPaintGeometry(FVector2f(2.f * CursorHalfLength, AllottedGeometry.GetLocalSize().Y), FSlateLayoutTransform(FVector2f(XPos - CursorHalfLength, 0.f)));

		FLinearColor CursorColor = InWidgetStyle.GetColorAndOpacityTint();
		CursorColor.A = CursorColor.A*0.08f;
		CursorColor.B *= 0.1f;
		CursorColor.G *= 0.2f;
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			CursorLayer,
			CursorGeometry,
			CursorBackground,
			DrawEffects,
			CursorColor
			);

		// Should draw above the text
		const int32 ArrowLayer = LayerId + 3;
		FPaintGeometry MyGeometry =	AllottedGeometry.ToPaintGeometry( FVector2f( HandleSize, AllottedGeometry.GetLocalSize().Y ) , FSlateLayoutTransform(FVector2f( XPos-HalfSize, 0 )));
		FLinearColor ScrubColor = InWidgetStyle.GetColorAndOpacityTint();

		// @todo Sequencer this color should be specified in the style
		ScrubColor.A = ScrubColor.A*0.5f;
		ScrubColor.B *= 0.1f;
		ScrubColor.G *= 0.2f;
		FSlateDrawElement::MakeBox( 
			OutDrawElements,
			ArrowLayer, 
			MyGeometry,
			bMirrorLabels ? ScrubHandleUp : ScrubHandleDown,
			DrawEffects, 
			ScrubColor
			);

		return ArrowLayer;
	}

	return LayerId;
}

FReply FVisualLoggerTimeSliderController::OnMouseButtonDown( SWidget& WidgetOwner, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	bool bHandleLeftMouseButton = MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton;
	bool bHandleRightMouseButton = MouseEvent.GetEffectingButton() == EKeys::RightMouseButton && TimeSliderArgs.AllowZoom;
	
	DistanceDragged = 0.;

	if ( bHandleLeftMouseButton )
	{
		// Always capture mouse if we left or right click on the widget
		const FScrubRangeToScreen RangeToScreen(TimeSliderArgs.ViewRange.Get(), MyGeometry.GetLocalSize());
		const FVector2f CursorPos = MyGeometry.AbsoluteToLocal(MouseEvent.GetLastScreenSpacePosition());
		const double NewValue = RangeToScreen.LocalXToInput(CursorPos.X);

		CommitScrubPosition(NewValue, /*bIsScrubbing=*/false);
		return FReply::Handled().CaptureMouse( WidgetOwner.AsShared() ).PreventThrottling();
	}
	else if ( bHandleRightMouseButton )
	{
		return FReply::Handled().CaptureMouse(WidgetOwner.AsShared());
	}

	return FReply::Unhandled();
}

FReply FVisualLoggerTimeSliderController::OnMouseButtonUp( SWidget& WidgetOwner, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	bool bHandleLeftMouseButton = MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton && WidgetOwner.HasMouseCapture();
	bool bHandleRightMouseButton = MouseEvent.GetEffectingButton() == EKeys::RightMouseButton && WidgetOwner.HasMouseCapture() && TimeSliderArgs.AllowZoom;
	
	if ( bHandleRightMouseButton )
	{
		if (!bPanning)
		{
			// return unhandled in case our parent wants to use our right mouse button to open a context menu
			return FReply::Unhandled().ReleaseMouseCapture();
		}
		
		bPanning = false;
		return FReply::Handled().ReleaseMouseCapture();
	}
	else if ( bHandleLeftMouseButton )
	{
		if( bDraggingScrubber )
		{
			TimeSliderArgs.OnEndScrubberMovement.ExecuteIfBound();
		}
		else
		{
			const FScrubRangeToScreen RangeToScreen( TimeSliderArgs.ViewRange.Get(), MyGeometry.GetLocalSize() );
			const FVector2f CursorPos = MyGeometry.AbsoluteToLocal(MouseEvent.GetLastScreenSpacePosition());
			const double NewValue = RangeToScreen.LocalXToInput(CursorPos.X);

			CommitScrubPosition( NewValue, /*bIsScrubbing=*/false );
		}

		bDraggingScrubber = false;
		return FReply::Handled().ReleaseMouseCapture();

	}

	return FReply::Unhandled();
}

double FVisualLoggerTimeSliderController::GetTimeAtCursorPosition(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) const
{
	const FScrubRangeToScreen RangeToScreen(TimeSliderArgs.ViewRange.Get(), MyGeometry.GetLocalSize());
	const FVector2f CursorPos = MyGeometry.AbsoluteToLocal(MouseEvent.GetLastScreenSpacePosition());
	double NewValue = RangeToScreen.LocalXToInput(CursorPos.X);

	const double LocalClampMin = TimeSliderArgs.ClampRange.Get().GetLowerBoundValue();
	const double LocalClampMax = TimeSliderArgs.ClampRange.Get().GetUpperBoundValue();

	if (NewValue < LocalClampMin)
	{
		NewValue = LocalClampMin;
	}

	if (NewValue > LocalClampMax)
	{
		NewValue = LocalClampMax;
	}

	return NewValue;
}

FReply FVisualLoggerTimeSliderController::OnMouseMove( SWidget& WidgetOwner, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if ( WidgetOwner.HasMouseCapture() )
	{
		if (MouseEvent.IsMouseButtonDown(EKeys::RightMouseButton))
		{
			if (!bPanning)
			{
				DistanceDragged += FMath::Abs( MouseEvent.GetCursorDelta().X );
				if ( DistanceDragged > FSlateApplication::Get().GetDragTriggerDistance() )
				{
					FReply::Handled().CaptureMouse(WidgetOwner.AsShared()).UseHighPrecisionMouseMovement(WidgetOwner.AsShared());
					SoftwareCursorPosition = MyGeometry.AbsoluteToLocal(MouseEvent.GetLastScreenSpacePosition());
					bPanning = true;
				}
			}
			else
			{
				SoftwareCursorPosition = MyGeometry.AbsoluteToLocal(MouseEvent.GetLastScreenSpacePosition());

				TRange<double> LocalViewRange = TimeSliderArgs.ViewRange.Get();
				double LocalViewRangeMin = LocalViewRange.GetLowerBoundValue();
				double LocalViewRangeMax = LocalViewRange.GetUpperBoundValue();

				FScrubRangeToScreen ScaleInfo( LocalViewRange, MyGeometry.GetLocalSize() );
				FVector2f ScreenDelta = MouseEvent.GetCursorDelta();
				FVector2D InputDelta;
				InputDelta.X = ScreenDelta.X/ScaleInfo.PixelsPerInput;

				const TRange<double> NewViewRange = TRange<double>(LocalViewRangeMin - InputDelta.X, LocalViewRangeMax - InputDelta.X);
				TRange<double> LocalClampRange = TimeSliderArgs.ClampRange.Get();

				// Do not try to pan outside the clamp range to prevent undesired zoom
				if (LocalClampRange.Contains(NewViewRange))
				{
					TimeSliderArgs.OnViewRangeChanged.ExecuteIfBound(NewViewRange);
					if (Scrollbar.IsValid())
					{
						double InOffsetFraction = (NewViewRange.GetLowerBoundValue() - LocalClampRange.GetLowerBoundValue()) / LocalClampRange.Size<double>();
						double InThumbSizeFraction = NewViewRange.Size<double>() / LocalClampRange.Size<double>();
						Scrollbar->SetState(static_cast<float>(InOffsetFraction), static_cast<float>(InThumbSizeFraction));
					}

					if (!TimeSliderArgs.ViewRange.IsBound())
					{
						// The  output is not bound to a delegate so we'll manage the value ourselves
						TimeSliderArgs.ViewRange.Set(NewViewRange);
					}
				}
			}
		}
		else if (MouseEvent.IsMouseButtonDown( EKeys::LeftMouseButton ))
		{
			if ( !bDraggingScrubber )
			{
				DistanceDragged += FMath::Abs( MouseEvent.GetCursorDelta().X );
				if ( DistanceDragged > 0. /*FSlateApplication::Get().GetDragTriggerDistance()*/ )
				{
					bDraggingScrubber = true;
					TimeSliderArgs.OnBeginScrubberMovement.ExecuteIfBound();
				}
			}
			else
			{
				const double NewValue = GetTimeAtCursorPosition(MyGeometry, MouseEvent);
				CommitScrubPosition(NewValue, /*bIsScrubbing=*/true);
			}
		}
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void FVisualLoggerTimeSliderController::CommitScrubPosition( double NewValue, bool bIsScrubbing )
{
	// Manage the scrub position ourselves if its not bound to a delegate
	if ( !TimeSliderArgs.ScrubPosition.IsBound() )
	{
		TimeSliderArgs.ScrubPosition.Set( NewValue );
	}

	TRange<double> LocalViewRange = TimeSliderArgs.ViewRange.Get();
	const double RangeSize = LocalViewRange.Size<double>();
	if (NewValue < LocalViewRange.GetLowerBoundValue())
	{
		SetTimeRange(NewValue, NewValue + RangeSize);
	}
	else if (NewValue > LocalViewRange.GetUpperBoundValue())
	{
		SetTimeRange(NewValue - RangeSize, NewValue);
	}

	TimeSliderArgs.OnScrubPositionChanged.ExecuteIfBound( NewValue, bIsScrubbing );
}

void FVisualLoggerTimeSliderController::SetExternalScrollbar(TSharedRef<SScrollBar> InScrollbar) 
{ 
	Scrollbar = InScrollbar;
	Scrollbar->SetOnUserScrolled(FOnUserScrolled::CreateRaw(this, &FVisualLoggerTimeSliderController::HorizontalScrollBar_OnUserScrolled)); 
};

void FVisualLoggerTimeSliderController::HorizontalScrollBar_OnUserScrolled(float ScrollOffset)
{
	if (!TimeSliderArgs.ViewRange.IsBound())
	{
		TRange<double> LocalViewRange = TimeSliderArgs.ViewRange.Get();
		double LocalViewRangeMin = LocalViewRange.GetLowerBoundValue();
		double LocalViewRangeMax = LocalViewRange.GetUpperBoundValue();
		double LocalClampMin = TimeSliderArgs.ClampRange.Get().GetLowerBoundValue();
		double LocalClampMax = TimeSliderArgs.ClampRange.Get().GetUpperBoundValue();

		double InThumbSizeFraction = (LocalViewRangeMax - LocalViewRangeMin) / (LocalClampMax - LocalClampMin);

		double NewViewOutputMin = LocalClampMin + ScrollOffset * (LocalClampMax - LocalClampMin);
		// The  output is not bound to a delegate so we'll manage the value ourselves
		double NewViewOutputMax = FMath::Min(NewViewOutputMin + (LocalViewRangeMax - LocalViewRangeMin), LocalClampMax);
		NewViewOutputMin = NewViewOutputMax - (LocalViewRangeMax - LocalViewRangeMin);

		double InOffsetFraction = (NewViewOutputMin - LocalClampMin) / (LocalClampMax - LocalClampMin);
		//if (InOffsetFraction + InThumbSizeFraction <= 1)
		{
			TimeSliderArgs.ViewRange.Set(TRange<double>(NewViewOutputMin, NewViewOutputMax));
			Scrollbar->SetState(static_cast<float>(InOffsetFraction), static_cast<float>(InThumbSizeFraction));
		}
	}
}

void FVisualLoggerTimeSliderController::SetTimeRange(double NewViewOutputMin, double NewViewOutputMax)
{
	TimeSliderArgs.ViewRange.Set(TRange<double>(NewViewOutputMin, NewViewOutputMax));

	double LocalClampMin = TimeSliderArgs.ClampRange.Get().GetLowerBoundValue();
	double LocalClampMax = TimeSliderArgs.ClampRange.Get().GetUpperBoundValue();

	const double InOffsetFraction = (NewViewOutputMin - LocalClampMin) / (LocalClampMax - LocalClampMin);
	const double InThumbSizeFraction = (NewViewOutputMax - NewViewOutputMin) / (LocalClampMax - LocalClampMin);
	Scrollbar->SetState(static_cast<float>(InOffsetFraction), static_cast<float>(InThumbSizeFraction));
}

void FVisualLoggerTimeSliderController::SetClampRange(double MinValue, double MaxValue)
{
	TRange<double> LocalViewRange = TimeSliderArgs.ViewRange.Get();
	double LocalClampMin = TimeSliderArgs.ClampRange.Get().GetLowerBoundValue();
	double LocalClampMax = TimeSliderArgs.ClampRange.Get().GetUpperBoundValue();
	const double CurrentDistance = LocalClampMax - LocalClampMin;
	const double ZoomDelta = (LocalViewRange.GetUpperBoundValue() - LocalViewRange.GetLowerBoundValue()) / CurrentDistance;

	MaxValue = MinValue + (MaxValue - MinValue < 2 ? CurrentDistance : MaxValue - MinValue);

	TimeSliderArgs.ClampRange = TRange<double>(MinValue, MaxValue);

	const double LocalViewRangeMin = FMath::Clamp(LocalViewRange.GetLowerBoundValue(), MinValue, MaxValue);
	const double LocalViewRangeMax = FMath::Clamp(LocalViewRange.GetUpperBoundValue(), MinValue, MaxValue);
	SetTimeRange(ZoomDelta >= 1 ? MinValue : LocalViewRangeMin, ZoomDelta >= 1 ? MaxValue : LocalViewRangeMax);
}

FReply FVisualLoggerTimeSliderController::OnMouseWheel( SWidget& WidgetOwner, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	FReply ReturnValue = FReply::Unhandled();

	if (MouseEvent.IsLeftShiftDown())
	{
		const float ZoomDelta = 0.025f * MouseEvent.GetWheelDelta();
		TimeSliderArgs.CursorSize.Set(FMath::Clamp(TimeSliderArgs.CursorSize.Get() + ZoomDelta, 0.0f, 1.0f));

		ReturnValue = FReply::Handled();
	}
	else if ( TimeSliderArgs.AllowZoom )
	{
		const float ZoomDelta = -0.1f * MouseEvent.GetWheelDelta();

		{
			double MouseFractionX = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition()).X / MyGeometry.GetLocalSize().X;

			TRange<double> LocalViewRange = TimeSliderArgs.ViewRange.Get();
			double LocalViewRangeMax = LocalViewRange.GetUpperBoundValue();
			double LocalViewRangeMin = LocalViewRange.GetLowerBoundValue();
			const double OutputViewSize = LocalViewRangeMax - LocalViewRangeMin;
			const double OutputChange = OutputViewSize * ZoomDelta;

			double NewViewOutputMin = LocalViewRangeMin - (OutputChange * MouseFractionX);
			double NewViewOutputMax = LocalViewRangeMax + (OutputChange * (1.0f - MouseFractionX));

			if (NewViewOutputMin < NewViewOutputMax)
			{
				double LocalClampMin = TimeSliderArgs.ClampRange.Get().GetLowerBoundValue();
				double LocalClampMax = TimeSliderArgs.ClampRange.Get().GetUpperBoundValue();

				// Clamp the range if clamp values are set
				if ( NewViewOutputMin < LocalClampMin )
				{
					NewViewOutputMin = LocalClampMin;
				}
				
				if ( NewViewOutputMax > LocalClampMax )
				{
					NewViewOutputMax = LocalClampMax;
				}

				TimeSliderArgs.OnViewRangeChanged.ExecuteIfBound(TRange<double>(NewViewOutputMin, NewViewOutputMax));
				if (Scrollbar.IsValid())
				{
					const double InOffsetFraction = (NewViewOutputMin - LocalClampMin) / (LocalClampMax - LocalClampMin);
					const double InThumbSizeFraction = (NewViewOutputMax - NewViewOutputMin) / (LocalClampMax - LocalClampMin);
					Scrollbar->SetState(static_cast<float>(InOffsetFraction), static_cast<float>(InThumbSizeFraction));
				}
				if( !TimeSliderArgs.ViewRange.IsBound() )
				{	
					// The  output is not bound to a delegate so we'll manage the value ourselves
					TimeSliderArgs.ViewRange.Set( TRange<double>( NewViewOutputMin, NewViewOutputMax ) );
				}
			}
		}

		ReturnValue = FReply::Handled();
	}

	return ReturnValue;
}

int32 FVisualLoggerTimeSliderController::OnPaintSectionView( const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, bool bEnabled, bool bDisplayTickLines, bool bDisplayScrubPosition  ) const
{
	const ESlateDrawEffect DrawEffects = bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

	TRange<double> LocalViewRange = TimeSliderArgs.ViewRange.Get();
	double LocalScrubPosition = TimeSliderArgs.ScrubPosition.Get();

	double ViewRange = LocalViewRange.Size<double>();
	double PixelsPerInput = ViewRange > 0. ? AllottedGeometry.GetLocalSize().X / ViewRange : 0.;
	double LinePos =  (LocalScrubPosition - LocalViewRange.GetLowerBoundValue()) * PixelsPerInput;

	FScrubRangeToScreen RangeToScreen( LocalViewRange, AllottedGeometry.GetLocalSize());

	if( bDisplayTickLines )
	{
		// Draw major tick lines in the section area
		FDrawTickArgs Args;
		Args.AllottedGeometry = AllottedGeometry;
		Args.bMirrorLabels = false;
		Args.bOnlyDrawMajorTicks = true;
		Args.TickColor = FLinearColor( 0.3f, 0.3f, 0.3f, 0.3f );
		Args.ClippingRect = MyCullingRect;
		Args.DrawEffects = DrawEffects;
		// Draw major ticks under sections
		Args.StartLayer = LayerId-1;
		// Draw the tick the entire height of the section area
		Args.TickOffset = 0.f;
		Args.MajorTickHeight = AllottedGeometry.GetLocalSize().Y;

		DrawTicks( OutDrawElements, RangeToScreen, Args );
	}

	if( bDisplayScrubPosition )
	{
		// Draw cursor size
		const float CursorHalfSize = TimeSliderArgs.CursorSize.Get() * 0.5f;
		const float CursorHalfLength = AllottedGeometry.GetLocalSize().X * CursorHalfSize;
		FPaintGeometry CursorGeometry = AllottedGeometry.ToPaintGeometry(FVector2f(2 * CursorHalfLength, AllottedGeometry.GetLocalSize().Y), FSlateLayoutTransform(FVector2f(LinePos - CursorHalfLength, 0)));

		FSlateDrawElement::MakeBox(
			OutDrawElements,
			++LayerId,
			CursorGeometry,
			CursorBackground,
			DrawEffects,
			FLinearColor::White.CopyWithNewOpacity(0.08f)
			);

		// Draw a line for the scrub position
		TArray<FVector2D> LinePoints;
		LinePoints.AddUninitialized(2);
		LinePoints[0] = FVector2D( 1., 0. );
		LinePoints[1] = FVector2D( 1., FMath::RoundToFloat( AllottedGeometry.GetLocalSize().Y ) );

		FSlateDrawElement::MakeLines(
			OutDrawElements,
			++LayerId,
			AllottedGeometry.ToPaintGeometry( FVector2f(1.0f,1.0f), FSlateLayoutTransform(FVector2f(LinePos, 0.0f )) ),
			LinePoints,
			DrawEffects,
			FLinearColor::White.CopyWithNewOpacity(0.39f),
			false
			);

	}

	return LayerId;
}

#undef LOCTEXT_NAMESPACE
