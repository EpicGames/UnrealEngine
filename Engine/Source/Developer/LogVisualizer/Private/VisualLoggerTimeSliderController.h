// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Input/CursorReply.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/Layout/SScrollBar.h"

class FSlateWindowElementList;

struct FVisualLoggerTimeSliderArgs
{
	DECLARE_DELEGATE_OneParam( FOnRangeChanged, TRange<double> )
	DECLARE_DELEGATE_TwoParams( FOnScrubPositionChanged, double, bool )

	FVisualLoggerTimeSliderArgs()
		: ScrubPosition(0.)
		, ViewRange(TRange<double>(0., 5.))
		, ClampRange(TRange<double>(0., 5.))
		, AllowZoom(true)
		, CursorSize(0.05f)
	{}

	/** The scrub position */
	TAttribute<double> ScrubPosition;

	/** Called when the scrub position changes */
	FOnScrubPositionChanged OnScrubPositionChanged;

	/** Called right before the scrubber begins to move */
	FSimpleDelegate OnBeginScrubberMovement;

	/** Called right after the scrubber handle is released by the user */
	FSimpleDelegate OnEndScrubberMovement;

	/** View time range */
	TAttribute< TRange<double> > ViewRange;

	/** Clamp time range */
	TAttribute< TRange<double> > ClampRange;

	/** Called when the view range changes */
	FOnRangeChanged OnViewRangeChanged;

	/** Attribute defining the time snap interval */
	TAttribute<double> TimeSnapInterval;

	/** If we are allowed to zoom */
	bool AllowZoom;

	/** Cursor range for data like histogram graphs, etc. */
	TAttribute< float > CursorSize;
};

/**
 * A time slider controller for sequencer
 * Draws and manages time data for a Sequencer
 */
class FVisualLoggerTimeSliderController
{
public:
	FVisualLoggerTimeSliderController(const FVisualLoggerTimeSliderArgs& InArgs);

	/**
	* Determines the optimal spacing between tick marks in the slider for a given pixel density
	* Increments until a minimum amount of slate units specified by MinTick is reached
	*
	* @param InPixelsPerInput	The density of pixels between each input
	* @param MinTick			The minimum slate units per tick allowed
	* @param MinTickSpacing	The minimum tick spacing in time units allowed
	* @return the optimal spacing in time units
	*/
	double DetermineOptimalSpacing(double InPixelsPerInput, uint32 MinTick, float MinTickSpacing) const;
	void SetTimesliderArgs(const FVisualLoggerTimeSliderArgs& InArgs);

	int32 OnPaintTimeSlider( bool bMirrorLabels, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const;
	FReply OnMouseButtonDown( SWidget& WidgetOwner, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent );
	FReply OnMouseButtonUp( SWidget& WidgetOwner, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent );
	FReply OnMouseMove( SWidget& WidgetOwner, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent );
	FReply OnMouseWheel( SWidget& WidgetOwner, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent );
	FCursorReply OnCursorQuery( TSharedRef<const SWidget> WidgetOwner, const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const { return FCursorReply::Unhandled(); }

	/**
	 * Draws major tick lines in the section view                                                              
	 */
	int32 OnPaintSectionView( const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, bool bEnabled, bool bDisplayTickLines, bool bDisplayScrubPosition ) const;
	FVisualLoggerTimeSliderArgs& GetTimeSliderArgs() { return TimeSliderArgs; }
	/**
	 * Call this method when the user's interaction has changed the scrub position
	 *
	 * @param NewValue				Value resulting from the user's interaction
	 * @param bIsScrubbing			True if done via scrubbing, false if just releasing scrubbing
	 */
	void CommitScrubPosition(double NewValue, bool bIsScrubbing );

	void SetExternalScrollbar(TSharedRef<SScrollBar> Scrollbar);
	void SetTimeRange(double MinValue, double MaxValue);
	void SetClampRange(double MinValue, double MaxValue);
	bool IsPanning() { return bPanning; }

	double GetTimeAtCursorPosition(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) const;
private:
	// forward declared as class members to prevent name collision with similar types defined in other units
	struct FScrubRangeToScreen;
	struct FDrawTickArgs;

	/**
	 * Draws time tick marks
	 *
	 * @param OutDrawElements	List to add draw elements to
	 * @param RangeToScreen		Time range to screen space converter
	 * @param InArgs			Parameters for drawing the tick lines
	 */
	void DrawTicks( FSlateWindowElementList& OutDrawElements, const FScrubRangeToScreen& RangeToScreen, FDrawTickArgs& InArgs ) const;
	void HorizontalScrollBar_OnUserScrolled(float ScrollOffset);
private:
	FVisualLoggerTimeSliderArgs TimeSliderArgs;
	/** Brush for drawing an upwards facing scrub handle */
	const FSlateBrush* ScrubHandleUp;
	/** Brush for drawing a downwards facing scrub handle */
	const FSlateBrush* ScrubHandleDown;
	/** Brush for drawing cursor background to visualize corsor size */
	const FSlateBrush* CursorBackground;
	/** Total mouse delta during dragging **/
	double DistanceDragged;
	/** If we are dragging the scrubber */
	bool bDraggingScrubber;
	/** If we are currently panning the panel */
	bool bPanning;
	/***/
	TSharedPtr<SScrollBar> Scrollbar;
	FVector2f SoftwareCursorPosition;
};
