// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

/* Dependencies
*****************************************************************************/
#include "CoreMinimal.h"

#define UE_API LOGVISUALIZER_API

class FText;
struct FGeometry;
struct FKeyEvent;
class FReply;
class FString;
class UWorld;
class UObject;

DECLARE_DELEGATE_OneParam(FOnFiltersSearchChanged, const FText&);

//DECLARE_DELEGATE(FOnFiltersChanged);
DECLARE_MULTICAST_DELEGATE(FOnFiltersChanged);

DECLARE_DELEGATE_ThreeParams(FOnLogLineSelectionChanged, TSharedPtr<struct FLogEntryItem> /*SelectedItem*/, int64 /*UserData*/, FName /*TagName*/);
DECLARE_DELEGATE_RetVal_TwoParams(FReply, FOnKeyboardEvent, const FGeometry& /*MyGeometry*/, const FKeyEvent& /*InKeyEvent*/);
DECLARE_DELEGATE_RetVal(float, FGetAnimationOutlinerFillPercentageFunc);

struct FVisualLoggerEvents
{
	FOnFiltersChanged OnFiltersChanged;
	FOnLogLineSelectionChanged OnLogLineSelectionChanged;
	FOnKeyboardEvent OnKeyboardEvent;
	FGetAnimationOutlinerFillPercentageFunc GetAnimationOutlinerFillPercentageFunc;
};

class FVisualLoggerTimeSliderController;
struct FLogVisualizer
{
	/** LogVisualizer interface*/
	UE_API void Reset();

	UE_API FLinearColor GetColorForCategory(int32 Index) const;
	UE_API FLinearColor GetColorForCategory(const FString& InFilterName) const;
	UE_API UWorld* GetWorld(UObject* OptionalObject = nullptr);
	FVisualLoggerEvents& GetEvents() { return VisualLoggerEvents; }

	void SetCurrentVisualizer(TSharedPtr<class SVisualLogger> Visualizer) { CurrentVisualizer = Visualizer; }

	void SetAnimationOutlinerFillPercentage(float FillPercentage) { AnimationOutlinerFillPercentage = FillPercentage; }
	float GetAnimationOutlinerFillPercentage()
	{
		if (VisualLoggerEvents.GetAnimationOutlinerFillPercentageFunc.IsBound())
		{
			SetAnimationOutlinerFillPercentage(VisualLoggerEvents.GetAnimationOutlinerFillPercentageFunc.Execute());
		}
		return AnimationOutlinerFillPercentage;
	}

	UE_API int32 GetNextItem(FName RowName, int32 MoveDistance = 1);
	UE_API int32 GetPreviousItem(FName RowName, int32 MoveDistance = 1);

	// @todo: This function currently doesn't do anything!
	UE_API void GotoNextItem(FName RowName, int32 MoveDistance = 1);
	// @todo: This function currently doesn't do anything!
	UE_API void GotoPreviousItem(FName RowName, int32 MoveDistance = 1);
	UE_API void GotoFirstItem(FName RowName);
	UE_API void GotoLastItem(FName RowName);
	UE_API void GotoLastItemAnyRow();

	UE_API void UpdateCameraPosition(FName Rowname, int32 ItemIndes);

	UE_API void SeekToTime(float Time);

	/** Static access */
	static UE_API void Initialize();
	static UE_API void Shutdown();
	static UE_API FLogVisualizer& Get();

protected:
	TSharedPtr<FVisualLoggerTimeSliderController> GetTimeSliderController() { return TimeSliderController; }

	static UE_API TSharedPtr< struct FLogVisualizer > StaticInstance;

	TSharedPtr<FVisualLoggerTimeSliderController> TimeSliderController;
	FVisualLoggerEvents VisualLoggerEvents;
	TWeakPtr<class SVisualLogger> CurrentVisualizer;
	float AnimationOutlinerFillPercentage;

	friend class SVisualLoggerViewer;
	friend class SVisualLoggerView;
	friend class SVisualLogger;
	friend struct FVisualLoggerCanvasRenderer;
};

#undef UE_API
