// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
GameplayDebuggerSettings.h: Declares the UGameplayDebuggerSettings class.
=============================================================================*/
#pragma once


#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "LogVisualizerSessionSettings.generated.h"

#define UE_API LOGVISUALIZER_API

UCLASS(MinimalAPI)
class ULogVisualizerSessionSettings : public UObject
{
	GENERATED_UCLASS_BODY()
public:
	/**Whether to show trivial logs, i.e. the ones with only one entry.*/
	UPROPERTY(EditAnywhere, Category = "VisualLogger")
	bool bEnableGraphsVisualization;

	DECLARE_EVENT_OneParam(ULogVisualizerSessionSettings, FSettingChangedEvent, FName /*PropertyName*/);
	FSettingChangedEvent& OnSettingChanged() { return SettingChangedEvent; }

	// UObject overrides
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

private:

	// Holds an event delegate that is executed when a setting has changed.
	FSettingChangedEvent SettingChangedEvent;
};

#undef UE_API
