// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
GameplayDebuggerSettings.h: Declares the UGameplayDebuggerSettings class.
=============================================================================*/
#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "CoreMinimal.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "VisualLogger/VisualLoggerTypes.h"
#include "LogVisualizerSettings.generated.h"

struct FVisualLoggerDBRow;

DECLARE_MULTICAST_DELEGATE_TwoParams(FOnFilterCategoryAdded, FString, ELogVerbosity::Type);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnFilterCategoryRemoved, FString);

struct FVisualLoggerDBRow;

USTRUCT()
struct FCategoryFilter
{
	GENERATED_BODY()

	FCategoryFilter()
		: LogVerbosity(ELogVerbosity::Type::NoLogging)
		, Enabled(0)
		, bIsInUse(0)
	{}

	UPROPERTY(config)
	FString CategoryName;

	UPROPERTY(config)
	int32 LogVerbosity;

	UPROPERTY(config)
	uint32 Enabled : 1;

	uint32 bIsInUse : 1;
};

USTRUCT()
struct FVisualLoggerFiltersData
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(config)
	FString SearchBoxFilter;
	
	UPROPERTY(config)
	FString ObjectNameFilter;
	
	UPROPERTY(config)
	TArray<FCategoryFilter> Categories;
	
	UPROPERTY(config)
	TArray<FString> SelectedClasses;
};

USTRUCT()
struct FVisualLoggerFilters : public FVisualLoggerFiltersData
{
	GENERATED_BODY()

#if UE_DEBUG_VISUALIZER_TOOL_ENABLED
	FOnFilterCategoryAdded OnFilterCategoryAdded;
	FOnFilterCategoryRemoved OnFilterCategoryRemoved;

	static FVisualLoggerFilters& Get();
	static void Initialize();
	static void Shutdown();

	void Reset();
	void InitWith(const FVisualLoggerFiltersData& NewFiltersData);

	/** @return whether given log line should be displayed */
	bool ShouldDisplayLine(const FVisualLogLine& Line, const bool bSearchInsideLogs) const;

	/**
	 * This is the preferred version to determine if a category should be displayed base on its name.
	 * @return whether given category name represents a log category we allow to be displayed at given Verbosity
	 */
	bool ShouldDisplayCategory(FName Name, ELogVerbosity::Type Verbosity = ELogVerbosity::All) const;

	/**
	 * @return whether given String represents a log category we allow to be displayed at given Verbosity
	 * @note this function relies on case-insensitive string comparison and is slower than the FName based version
	 * that should be used when the category name is directly accessible.
	 * @see ShouldDisplayCategory
	 */
	bool ShouldDisplayCategoryByString(FStringView String, ELogVerbosity::Type Verbosity = ELogVerbosity::All) const;

	/** @return whether given String is a case-insensitive match to the active search filter */
	bool IsStringMatchingSearchFilter(const FStringView String) const
	{
		return SearchBoxFilter.Equals(String.GetData(), ESearchCase::IgnoreCase);
	}

	void SetSearchString(FString InString)
	{
		SearchBoxFilter = MoveTemp(InString);
	}

	FString GetSearchString() const
	{
		return SearchBoxFilter;
	}

	void AddCategory(FString InName, ELogVerbosity::Type InVerbosity);
	void RemoveCategory(FString InName);
	FCategoryFilter& GetCategoryByName(const FString& InName);
	FCategoryFilter& GetCategoryByName(const FName& InName);

	void DeactivateAllButThis(const FString& InName);
	void EnableAllCategories();

	bool MatchObjectName(FString String);
	void SelectObject(FString ObjectName);
	void RemoveObjectFromSelection(FString ObjectName);
	const TArray<FString>& GetSelectedObjects() const;

	void DisableGraphData(FName GraphName, FName DataName, bool SetAsDisabled);
	bool IsGraphDataDisabled(FName GraphName, FName DataName);

protected:
	void OnNewItemHandler(const FVisualLoggerDBRow& BDRow, int32 ItemIndex);

private:
	static TSharedPtr<FVisualLoggerFilters> StaticInstance;
	TMap<FName, FCategoryFilter*> FastCategoryFilterMap;
	TArray<FName> DisabledGraphDatas;

#endif //UE_DEBUG_VISUALIZER_TOOL_ENABLED
};

struct FCategoryFiltersManager;

UCLASS(MinimalAPI, config = EditorPerProjectUserSettings)
class ULogVisualizerSettings : public UObject
{
	GENERATED_UCLASS_BODY()
	friend struct FCategoryFiltersManager;

public:
	DECLARE_EVENT_OneParam(ULogVisualizerSettings, FSettingChangedEvent, FName /*PropertyName*/);
	FSettingChangedEvent& OnSettingChanged()
	{
		return SettingChangedEvent;
	}

	/**Whether to show trivial logs, i.e. the ones with only one entry.*/
	UPROPERTY(EditAnywhere, config, Category = "VisualLogger")
	bool bIgnoreTrivialLogs;

	/**Threshold for trivial Logs*/
	UPROPERTY(EditAnywhere, config, Category = "VisualLogger", meta = (EditCondition = "bIgnoreTrivialLogs", ClampMin = "0", ClampMax = "10", UIMin = "0", UIMax = "10"))
	int32 TrivialLogsThreshold;

	UE_DEPRECATED(5.6, "This is now controlled by the auto-scroll button in the tool")
	UPROPERTY()
	bool bStickToRecentData;

	/**Whether to reset current data or not for each new session.*/
	UPROPERTY(EditAnywhere, config, Category = "VisualLogger")
	bool bResetDataWithNewSession;

	/**Whether to show histogram labels inside graph or outside. Property disabled for now.*/
	UPROPERTY(VisibleAnywhere, config, Category = "VisualLogger")
	bool bShowHistogramLabelsOutside;

	/** Camera distance used to setup location during reaction on log item double click */
	UPROPERTY(EditAnywhere, config, Category = "VisualLogger", meta = (ClampMin = "10", ClampMax = "1000", UIMin = "10", UIMax = "1000"))
	float DefaultCameraDistance;

	/**Whether to search/filter categories or to get text vlogs into account too */
	UPROPERTY(EditAnywhere, config, Category = "VisualLogger")
	bool bSearchInsideLogs;

	/** Whether to only show events occuring within one of visual logger filter volumes currently in the level */
	UPROPERTY(EditAnywhere, config, Category = "VisualLogger")
	bool bUseFilterVolumes;

	/** Background color for 2d graphs visualization */
	UPROPERTY(EditAnywhere, config, Category = "VisualLogger")
	FColor GraphsBackgroundColor;

	/**Whether to store all filter settings on exit*/
	UPROPERTY(EditAnywhere, config, Category = "VisualLogger")
	bool bPersistentFilters;

	/** Whether to extreme values on graph (data has to be provided for extreme values) */
	UPROPERTY(EditAnywhere, config, Category = "VisualLogger")
	bool bDrawExtremesOnGraphs;

	/** Graphs will be scaled around local Min/Max values (values being displayed) rather than all historic data */
	UPROPERTY(EditAnywhere, config, Category = "VisualLogger")
	bool bConstrainGraphToLocalMinMax;

	/** Whether to use PlayersOnly during Pause or not */
	UPROPERTY(EditAnywhere, config, Category = "VisualLogger")
	bool bUsePlayersOnlyForPause;

	/** Whether to dump Navigation Octree on Stop recording or not */
	UPROPERTY(EditAnywhere, config, Category = "VisualLogger")
	bool bLogNavOctreeOnStop;

	/** controls how we generate log names. When set to TRUE there's a lot lower 
	 *	chance of name conflict, but it's more expensive */
	UPROPERTY(EditAnywhere, config, Category = "VisualLogger")
	bool bForceUniqueLogNames;

	// UObject overrides
#if WITH_EDITOR
	LOGVISUALIZER_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	LOGVISUALIZER_API class UMaterial* GetDebugMeshMaterial();

	LOGVISUALIZER_API void SavePersistentData();

	LOGVISUALIZER_API void ClearPersistentData();

	LOGVISUALIZER_API void LoadPersistentData();

	LOGVISUALIZER_API void ConfigureVisLog();

protected:
	UPROPERTY(config)
	FVisualLoggerFiltersData PersistentFilters;

	/** A material used to render debug meshes with kind of flat shading, mostly used by Visual Logger tool. */
	UPROPERTY()
	TObjectPtr<UMaterial> DebugMeshMaterialFakeLight;

	/** @todo document */
	UPROPERTY(config)
	FString DebugMeshMaterialFakeLightName;
private:

	// Holds an event delegate that is executed when a setting has changed.
	FSettingChangedEvent SettingChangedEvent;

};
