// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "VisualLogger/VisualLoggerTypes.h"
#include "DebugRenderSceneProxy.h"
#include "VisualLoggerRenderingActorBase.generated.h"

#define UE_API LOGVISUALIZER_API

class UVisualLoggerRenderingComponent;

/**
*	Transient actor used to draw visual logger data on level
*	Base class shared between RewindDebugger and Visual Logger implementations
*/

UCLASS(MinimalAPI, config = Engine, NotBlueprintable, Transient, notplaceable, AdvancedClassDisplay)
class AVisualLoggerRenderingActorBase : public AActor
{
public:
	GENERATED_UCLASS_BODY()
	UE_API virtual ~AVisualLoggerRenderingActorBase();

	struct FTimelineDebugShapes
	{
		TArray<FDebugRenderSceneProxy::FDebugLine> Lines;
		TArray<FDebugRenderSceneProxy::FCircle> Circles;
		TArray<FDebugRenderSceneProxy::FCone> Cones;
		TArray<FDebugRenderSceneProxy::FDebugBox> Boxes;
		TArray<FDebugRenderSceneProxy::FSphere> Points;
		TArray<FDebugRenderSceneProxy::FMesh> Meshes;
		TArray<FDebugRenderSceneProxy::FText3d> Texts;
		TArray<FDebugRenderSceneProxy::FWireCylinder> Cylinders;
		TArray<FDebugRenderSceneProxy::FCapsule> Capsules;
		TArray<FDebugRenderSceneProxy::FArrowLine> Arrows;
		TArray<FDebugRenderSceneProxy::FCoordinateSystem> CoordinateSystems;
		TArray<FVector> LogEntriesPath;

		void Reset()
		{
			Lines.Reset();
			Circles.Reset();
			Cones.Reset();
			Boxes.Reset();
			Points.Reset();
			Meshes.Reset();
			Texts.Reset();
			Cylinders.Reset();
			Capsules.Reset();
			Arrows.Reset();
			CoordinateSystems.Reset();
			LogEntriesPath.Reset();
		}
	};

	// Iterate over each active FTimelineDebugShapes, and call callback
	virtual void IterateDebugShapes(TFunction<void(const FTimelineDebugShapes&) > Callback) { };
	virtual bool MatchCategoryFilters(const FName& CategoryName, ELogVerbosity::Type Verbosity) const { return true; };

#if WITH_EDITOR
	virtual bool IsSelectable() const override { return false; }
#endif // WITH_EDITOR

// Allows to override the far clipping distance for all the log elements by this actor :
	UE_API void SetFarClippingDistance(double Distance);
	UE_API double GetFarClippingDistance() const;

protected:
#if UE_DEBUG_VISUALIZER_TOOL_ENABLED
	UE_API void GetDebugShapes(const FVisualLogEntry& EntryItem, bool bAddEntryLocationPointer, FTimelineDebugShapes& OutDebugShapes);
#endif

	UVisualLoggerRenderingComponent* RenderingComponent = nullptr;
};

#undef UE_API
